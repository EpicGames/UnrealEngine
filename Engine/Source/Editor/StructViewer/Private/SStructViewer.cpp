// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStructViewer.h"
#include "StructViewerNode.h"
#include "StructViewerFilter.h"
#include "StructViewerProjectSettings.h"

#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Misc/TextFilterExpressionEvaluator.h"

#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "SlateOptMacros.h"
#include "EditorWidgetsModule.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"

#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Framework/Docking/TabManager.h"
#include "SListViewSelectorDropdownMenu.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "ContentBrowserDataDragDropOp.h"

#include "Editor/UnrealEdEngine.h"
#include "StructUtils/UserDefinedStruct.h"
#include "EditorDirectories.h"
#include "Dialogs/Dialogs.h"

#include "IDocumentation.h"
#include "PropertyHandle.h"
#include "Logging/LogMacros.h"
#include "SourceCodeNavigation.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "SStructViewer"

DEFINE_LOG_CATEGORY_STATIC(LogEditorStructViewer, Log, All);

EStructFilterReturn FStructViewerFilterFuncs::IfInChildOfStructsSet(TSet<const UScriptStruct*>& InSet, const UScriptStruct* InStruct)
{
	check(InStruct);

	if (InSet.Num())
	{
		// If a struct is a child of any structs on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (const UScriptStruct* CurStruct : InSet)
		{
			if (InStruct->IsChildOf(CurStruct))
			{
				return EStructFilterReturn::Passed;
			}
		}

		return EStructFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EStructFilterReturn::NoItems;
}

EStructFilterReturn FStructViewerFilterFuncs::IfMatchesAllInChildOfStructsSet(TSet<const UScriptStruct*>& InSet, const UScriptStruct* InStruct)
{
	check(InStruct);

	if (InSet.Num())
	{
		// If a struct is a child of any structs on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (const UScriptStruct* CurStruct : InSet)
		{
			if (!InStruct->IsChildOf(CurStruct))
			{
				// Since it doesn't match one, it fails.
				return EStructFilterReturn::Failed;
			}
		}

		// It matches all of them, so it passes.
		return EStructFilterReturn::Passed;
	}

	// Since there are none on this list, return that there is no items.
	return EStructFilterReturn::NoItems;
}

EStructFilterReturn FStructViewerFilterFuncs::IfInStructsSet(TSet<const UScriptStruct*>& InSet, const UScriptStruct* InStruct)
{
	check(InStruct);

	if (InSet.Num())
	{
		return InSet.Contains(InStruct)
			? EStructFilterReturn::Passed
			: EStructFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EStructFilterReturn::NoItems;
}

class FStructHierarchy
{
public:
	FStructHierarchy();
	~FStructHierarchy();

	/** Get the singleton instance, creating it if required */
	static FStructHierarchy& Get();

	/** Get the singleton instance, or null if it doesn't exist */
	static FStructHierarchy* GetPtr();

	/** Destroy the singleton instance */
	static void DestroyInstance();

	/** Used to inform any registered Struct Viewers to refresh. */
	DECLARE_MULTICAST_DELEGATE(FPopulateStructViewer);
	FPopulateStructViewer& GetPopulateStructViewerDelegate()
	{
		return PopulateStructViewerDelegate;
	}

	/** Get the root node of the tree */
	TSharedRef<FStructViewerNodeData> GetStructRootNode() const
	{
		return StructRootNode.ToSharedRef();
	}
	
	/**
	 * Finds the node, recursively going deeper into the hierarchy.
	 * @param InRootNode	The node to start the search from.
	 * @param InStructPath	The path name of the struct to find the node for.
	 * @return The node.
	 */
	TSharedPtr<FStructViewerNodeData> FindNodeByStructPath(const TSharedRef<FStructViewerNodeData>& InRootNode, const FSoftObjectPath& InStructPath);
	TSharedPtr<FStructViewerNodeData> FindNodeByStructPath(const FSoftObjectPath& InStructPath)
	{
		return FindNodeByStructPath(GetStructRootNode(), InStructPath);
	}

	/** Update the struct hierarchy if it is pending a refresh */
	void UpdateStructHierarchy();

private:
	/** Dirty the struct hierarchy so it will be rebuilt on the next call to UpdateStructHierarchy */
	void DirtyStructHierarchy();

	/** Populates the struct hierarchy tree, pulling in all the loaded and unloaded structs. */
	void PopulateStructHierarchy();

	/**
	 * Recursively searches through the hierarchy to find and remove the asset. Used when deleting assets.
	 *
	 * @param InRootNode	The node to start the search from.
	 * @param InStructPath	The path name of the struct to delete
	 *
	 * @return Returns true if the struct was found and deleted successfully.
	 */
	bool FindAndRemoveNodeByStructPath(const TSharedRef<FStructViewerNodeData>& InRootNode, const FSoftObjectPath& InStructPath);
	bool FindAndRemoveNodeByStructPath(const FSoftObjectPath& InStructPath)
	{
		return FindAndRemoveNodeByStructPath(GetStructRootNode(), InStructPath);
	}

	/** Callback registered to the Asset Registry to be notified when an asset is added. */
	void AddAsset(const FAssetData& InAddedAssetData);

	/** Callback registered to the Asset Registry to be notified when an asset is removed. */
	void RemoveAsset(const FAssetData& InRemovedAssetData);

	/** Called when reload has finished */
	void OnReloadComplete(EReloadCompleteReason Reason);

	/** Called when modules are loaded or unloaded */
	void OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange);

private:
	/** The struct hierarchy singleton that manages the unfiltered struct tree for the Struct Viewer. */
	static TUniquePtr<FStructHierarchy> Singleton;

	/** True if the struct hierarchy should be refreshed */
	bool bRefreshStructHierarchy = false;

	/** Used to inform any registered Struct Viewers to refresh. */
	FPopulateStructViewer PopulateStructViewerDelegate;

	/** The dummy struct data node that is used as a root point for the Struct Viewer. */
	TSharedPtr<FStructViewerNodeData> StructRootNode;
};

TUniquePtr<FStructHierarchy> FStructHierarchy::Singleton;

namespace StructViewer
{
	namespace Helpers
	{
		/**
		 * Checks if the struct is allowed under the init options of the struct viewer currently building it's tree/list.
		 * @param InInitOptions		The struct viewer's options, holds the AllowedStructs and DisallowedStructs.
		 * @param InStruct			The struct to test against.
		 */
		bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const TWeakObjectPtr<const UScriptStruct> InStruct)
		{
			if (InInitOptions.StructFilter.IsValid())
			{
				return InInitOptions.StructFilter->IsStructAllowed(InInitOptions, InStruct.Get(), MakeShared<FStructViewerFilterFuncs>());
			}
			return true;
		}

		/**
		 * Checks if the unloaded struct is allowed under the init options of the struct viewer currently building it's tree/list.
		 * @param InInitOptions		The struct viewer's options, holds the AllowedStructs and DisallowedStructs.
		 * @param InStructPath		The path name to test against.
		 */
		bool IsStructAllowed_UnloadedStruct(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath)
		{
			if (InInitOptions.StructFilter.IsValid())
			{
				return InInitOptions.StructFilter->IsUnloadedStructAllowed(InInitOptions, InStructPath, MakeShared<FStructViewerFilterFuncs>());
			}
			return true;
		}

		/**
		 * Checks if the TestString passes the filter.
		 * @param InTestString		The string to test against the filter.
		 * @param InTextFilter		Compiled text filter to apply.
		 * @return	true if it passes the filter.
		 */
		bool PassesFilter(const FString& InTestString, const FTextFilterExpressionEvaluator& InTextFilter)
		{
			return InTextFilter.TestTextFilter(FBasicStringFilterExpressionContext(InTestString));
		}

		/**
		 * Recursive function to build a tree, filtering out nodes based on the InitOptions and filter search terms.
		 * @param InInitOptions						The struct viewer's options, holds the AllowedStructs and DisallowedStructs.
		 * @param InOutRootNode						The node that this function will add the children of to the tree.
		 * @param InOriginalRootNode				The original root node holding the data we produce a filtered node for.
		 * @param InTextFilter						Compiled text filter to apply.
		 * @param bInShowUnloadedStructs			Filter option to not remove unloaded structs due to struct filter options.
		 * @param InAllowedDeveloperType            Filter option for dealing with developer folders.
		 * @param bInInternalStructs                Filter option for showing internal structs.
		 * @param InternalStructs                   The structs that have been marked as Internal Only.
		 * @param InternalPaths                     The paths that have been marked Internal Only.
		 * @return Returns true if the child passed the filter.
		 */
		bool AddChildren_Tree(
			const FStructViewerInitializationOptions& InInitOptions, 
			const TSharedRef<FStructViewerNode>& InOutRootNode,
			const TSharedRef<FStructViewerNodeData>& InOriginalRootNode, 
			const FTextFilterExpressionEvaluator& InTextFilter,
			const bool bInShowUnloadedStructs, 
			const EStructViewerDeveloperType InAllowedDeveloperType,
			const bool bInInternalStructs,
			const TArray<const UScriptStruct*>& InternalStructs,
			const TArray<FDirectoryPath>& InternalPaths
			)
		{
			static const FString DeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir());
			static const FString UserDeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameUserDeveloperDir());

			const UScriptStruct* OriginalRootNodeStruct = InOriginalRootNode->GetStruct();

			// Determine if we allow any developer folder classes, if so determine if this struct is in one of the allowed developer folders.
			const FString GeneratedStructPathString = InOriginalRootNode->GetStructPath().ToString();
			bool bPassesDeveloperFilter = true;
			if (InAllowedDeveloperType == EStructViewerDeveloperType::SVDT_None)
			{
				bPassesDeveloperFilter = !GeneratedStructPathString.StartsWith(DeveloperPathWithSlash);
			}
			else if (InAllowedDeveloperType == EStructViewerDeveloperType::SVDT_CurrentUser)
			{
				if (GeneratedStructPathString.StartsWith(DeveloperPathWithSlash))
				{
					bPassesDeveloperFilter = GeneratedStructPathString.StartsWith(UserDeveloperPathWithSlash);
				}
			}

			// The INI files declare structs and folders that are considered internal only. Does this struct match any of those patterns?
			// INI path: /Script/StructViewer.StructViewerProjectSettings
			bool bPassesInternalFilter = true;
			if (!bInInternalStructs && InternalPaths.Num() > 0)
			{
				for (const FDirectoryPath& InternalPath : InternalPaths)
				{
					if (GeneratedStructPathString.StartsWith(InternalPath.Path))
					{
						bPassesInternalFilter = false;
						break;
					}
				}
			}
			if (!bInInternalStructs && InternalStructs.Num() > 0 && bPassesInternalFilter && OriginalRootNodeStruct)
			{
				for (const UScriptStruct* InternalStruct : InternalStructs)
				{
					if (OriginalRootNodeStruct->IsChildOf(InternalStruct))
					{
						bPassesInternalFilter = false;
						break;
					}
				}
			}

			// There are few options for filtering an unloaded struct, if it matches with this filter, it passes.
			bool bReturnPassesFilter = false;
			if (OriginalRootNodeStruct)
			{
				bReturnPassesFilter = bPassesDeveloperFilter && bPassesInternalFilter && IsStructAllowed(InInitOptions, OriginalRootNodeStruct) &&
					(PassesFilter(InOriginalRootNode->GetStructName(), InTextFilter) || PassesFilter(InOriginalRootNode->GetStructDisplayName().ToString(), InTextFilter));
			}
			else
			{
				if (bInShowUnloadedStructs)
				{
					bReturnPassesFilter = bPassesDeveloperFilter && bPassesInternalFilter && IsStructAllowed_UnloadedStruct(InInitOptions, InOutRootNode->GetStructPath()) &&
						(PassesFilter(InOriginalRootNode->GetStructName(), InTextFilter) || PassesFilter(InOriginalRootNode->GetStructDisplayName().ToString(), InTextFilter));
				}
			}
			InOutRootNode->PassedFilter(bReturnPassesFilter);

			for (const TSharedPtr<FStructViewerNodeData>& ChildNode : InOriginalRootNode->GetChildNodes())
			{
				TSharedRef<FStructViewerNode> NewNode = MakeShared<FStructViewerNode>(ChildNode.ToSharedRef(), InInitOptions.PropertyHandle, /*bPassedFilter*/false); // Whether we pass the filter is calculated in the recursive call

				const bool bChildrenPassesFilter = AddChildren_Tree(InInitOptions, NewNode, ChildNode.ToSharedRef(), InTextFilter, bInShowUnloadedStructs, InAllowedDeveloperType, bInInternalStructs, InternalStructs, InternalPaths);
				bReturnPassesFilter |= bChildrenPassesFilter;

				if (bChildrenPassesFilter)
				{
					InOutRootNode->AddChild(NewNode);
				}
			}

			return bReturnPassesFilter;
		}

		/**
		 * Builds the struct tree.
		 * @param InInitOptions						The struct viewer's options, holds the AllowedStructs and DisallowedStructs.
		 * @param InOutRootNode						The node to root the tree to.
		 * @param InTextFilter						Compiled text filter to apply.
		 * @param bInShowUnloadedStructs			Filter option to not remove unloaded structs due to struct filter options.
		 * @param InAllowedDeveloperType            Filter option for dealing with developer folders.
		 * @param bInInternalStructs                Filter option for showing internal structs.
		 * @param InternalStructs                   The structs that have been marked as Internal Only.
		 * @param InternalPaths                     The paths that have been marked Internal Only.
		 * @return A fully built tree.
		 */
		void GetStructTree(
			const FStructViewerInitializationOptions& InInitOptions, 
			TSharedPtr<FStructViewerNode>& InOutRootNode,
			const FTextFilterExpressionEvaluator& InTextFilter, 
			const bool bInShowUnloadedStructs,
			const EStructViewerDeveloperType InAllowedDeveloperType = EStructViewerDeveloperType::SVDT_All,
			const bool bInInternalStructs = true,
			const TArray<const UScriptStruct*>& InternalStructs = TArray<const UScriptStruct*>(),
			const TArray<FDirectoryPath>& InternalPaths = TArray<FDirectoryPath>()
			)
		{
			const TSharedRef<FStructViewerNodeData> StructRootNode = FStructHierarchy::Get().GetStructRootNode();

			// Make a dummy root node
			InOutRootNode = MakeShared<FStructViewerNode>();

			AddChildren_Tree(InInitOptions, InOutRootNode.ToSharedRef(), StructRootNode, InTextFilter, bInShowUnloadedStructs, InAllowedDeveloperType, bInInternalStructs, InternalStructs, InternalPaths);
		}

		/**
		 * Recursive function to build the list, filtering out nodes based on the InitOptions and filter search terms.
		 * @param InInitOptions						The struct viewer's options, holds the AllowedStructs and DisallowedStructs.
		 * @param InOutNodeList						The list to add all the nodes to.
		 * @param InOriginalRootNode				The original root node holding the data we produce a filtered node for.
		 * @param InTextFilter						Compiled text filter to apply.
		 * @param bInShowUnloadedStructs			Filter option to not remove unloaded structs due to struct filter options.
		 * @param InAllowedDeveloperType            Filter option for dealing with developer folders.
		 * @param bInInternalStructs                Filter option for showing internal structs.
		 * @param InternalStructs                   The structs that have been marked as Internal Only.
		 * @param InternalPaths                     The paths that have been marked Internal Only.
		 * @return Returns true if the child passed the filter.
		 */
		void AddChildren_List(
			const FStructViewerInitializationOptions& InInitOptions, 
			TArray<TSharedPtr<FStructViewerNode>>& InOutNodeList,
			const TSharedRef<FStructViewerNodeData>& InOriginalRootNode, 
			const FTextFilterExpressionEvaluator& InTextFilter,
			const bool bInShowUnloadedStructs,
			const EStructViewerDeveloperType InAllowedDeveloperType,
			const bool bInInternalStructs,
			const TArray<const UScriptStruct*>& InternalStructs, 
			const TArray<FDirectoryPath>& InternalPaths
			)
		{
			static const FString DeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir());
			static const FString UserDeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameUserDeveloperDir());

			const UScriptStruct* OriginalRootNodeStruct = InOriginalRootNode->GetStruct();

			// Determine if we allow any developer folder structs, if so determine if this struct is in one of the allowed developer folders.
			FString GeneratedStructPathString = InOriginalRootNode->GetStructPath().ToString();
			bool bPassesDeveloperFilter = true;
			if (InAllowedDeveloperType == EStructViewerDeveloperType::SVDT_None)
			{
				bPassesDeveloperFilter = !GeneratedStructPathString.StartsWith(DeveloperPathWithSlash);
			}
			else if (InAllowedDeveloperType == EStructViewerDeveloperType::SVDT_CurrentUser)
			{
				if (GeneratedStructPathString.StartsWith(DeveloperPathWithSlash))
				{
					bPassesDeveloperFilter = GeneratedStructPathString.StartsWith(UserDeveloperPathWithSlash);
				}
			}

			// The INI files declare structs and folders that are considered internal only. Does this struct match any of those patterns?
			// INI path: /Script/StructViewer.StructViewerProjectSettings
			bool bPassesInternalFilter = true;
			if (!bInInternalStructs && InternalPaths.Num() > 0)
			{
				for (const FDirectoryPath& InternalPath : InternalPaths)
				{
					if (GeneratedStructPathString.StartsWith(InternalPath.Path))
					{
						bPassesInternalFilter = false;
						break;
					}
				}
			}
			if (!bInInternalStructs && InternalStructs.Num() > 0 && bPassesInternalFilter && OriginalRootNodeStruct)
			{
				for (const UScriptStruct* InternalStruct : InternalStructs)
				{
					if (OriginalRootNodeStruct->IsChildOf(InternalStruct))
					{
						bPassesInternalFilter = false;
						break;
					}
				}
			}

			// There are few options for filtering an unloaded struct, if it matches with this filter, it passes.
			bool bPassedFilter = false;
			if (OriginalRootNodeStruct)
			{
				bPassedFilter = bPassesDeveloperFilter && bPassesInternalFilter && IsStructAllowed(InInitOptions, OriginalRootNodeStruct) &&
					(PassesFilter(InOriginalRootNode->GetStructName(), InTextFilter) || PassesFilter(InOriginalRootNode->GetStructDisplayName().ToString(), InTextFilter));
			}
			else
			{
				if (bInShowUnloadedStructs)
				{
					bPassedFilter = bPassesDeveloperFilter && bPassesInternalFilter && IsStructAllowed_UnloadedStruct(InInitOptions, InOriginalRootNode->GetStructPath()) &&
						(PassesFilter(InOriginalRootNode->GetStructName(), InTextFilter) || PassesFilter(InOriginalRootNode->GetStructDisplayName().ToString(), InTextFilter));
				}
			}

			if (bPassedFilter)
			{
				InOutNodeList.Add(MakeShared<FStructViewerNode>(InOriginalRootNode, InInitOptions.PropertyHandle, bPassedFilter));
			}

			for (const TSharedPtr<FStructViewerNodeData>& ChildNode : InOriginalRootNode->GetChildNodes())
			{
				AddChildren_List(InInitOptions, InOutNodeList, ChildNode.ToSharedRef(), InTextFilter, bInShowUnloadedStructs, InAllowedDeveloperType, bInInternalStructs, InternalStructs, InternalPaths);
			}
		}

		/**
		 * Builds the struct list.
		 * @param InInitOptions						The struct viewer's options, holds the AllowedStructs and DisallowedStructs.
		 * @param InOutNodeList						The list to add all the nodes to.
		 * @param InTextFilter						Compiled text filter to apply.
		 * @param bInShowUnloadedStructs			Filter option to not remove unloaded structs due to struct filter options.
		 * @param InAllowedDeveloperType            Filter option for dealing with developer folders.
		 * @param bInInternalStructs                Filter option for showing internal structs.
		 * @param InternalStructs                   The structs that have been marked as Internal Only.
		 * @param InternalPaths                     The paths that have been marked Internal Only.
		 * @return A fully built list.
		 */
		void GetStructList(
			const FStructViewerInitializationOptions& InInitOptions, 
			TArray<TSharedPtr<FStructViewerNode>>& InOutNodeList,
			const FTextFilterExpressionEvaluator& InTextFilter, 
			const bool bInShowUnloadedStructs,
			const EStructViewerDeveloperType InAllowedDeveloperType = EStructViewerDeveloperType::SVDT_All,
			const bool bInInternalStructs = true,
			const TArray<const UScriptStruct*>& InternalStructs = TArray<const UScriptStruct*>(), 
			const TArray<FDirectoryPath>& InternalPaths = TArray<FDirectoryPath>()
			)
		{
			const TSharedRef<FStructViewerNodeData> StructRootNode = FStructHierarchy::Get().GetStructRootNode();

			// Always skip the dummy root, only adding its children
			for (const TSharedPtr<FStructViewerNodeData>& ChildNode : StructRootNode->GetChildNodes())
			{
				AddChildren_List(InInitOptions, InOutNodeList, ChildNode.ToSharedRef(), InTextFilter, bInShowUnloadedStructs, InAllowedDeveloperType, bInInternalStructs, InternalStructs, InternalPaths);
			}
		}

		/**
		 * Opens an asset editor for a user defined struct.
		 */
		void OpenAssetEditor(const UUserDefinedStruct* InStruct)
		{
			if (InStruct)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(const_cast<UUserDefinedStruct*>(InStruct));
			}
		}

		/**
		 * Opens a struct source file.
		 */
		void OpenStructInIDE(const UScriptStruct* InStruct)
		{
			FSourceCodeNavigation::NavigateToStruct(InStruct);
		}

		/**
		 * Copy struct FTopLevelAssetPath to clipboard.
		 */
		void CopyStructPathToClipboard(const UScriptStruct* InStruct)
		{
			if (InStruct)
			{
				FPlatformApplicationMisc::ClipboardCopy(*InStruct->GetStructPathName().ToString());
			}
		}

		/**
		 * Finds the struct in the content browser.
		 */
		void FindInContentBrowser(const UScriptStruct* InStruct)
		{
			if (InStruct)
			{
				TArray<UObject*> Objects;
				Objects.Add(const_cast<UScriptStruct*>(InStruct));
				GEditor->SyncBrowserToObjects(Objects);
			}
		}

		TSharedRef<SWidget> CreateMenu(const UScriptStruct* InStruct)
		{
			// Empty list of commands.
			TSharedPtr<FUICommandList> Commands;

			const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Commands);
			{
				if (const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InStruct))
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("StructViewerMenuEditStructAsset", "Edit Struct..."), 
						LOCTEXT("StructViewerMenuEditStructAsset_Tooltip", "Open the struct in the asset editor."), 
						FSlateIcon(), 
						FUIAction(FExecuteAction::CreateStatic(&StructViewer::Helpers::OpenAssetEditor, UserDefinedStruct))
						);

					MenuBuilder.AddMenuEntry(
						LOCTEXT("StructViewerMenuFindContent", "Find in Content Browser..."), 
						LOCTEXT("StructViewerMenuFindContent_Tooltip", "Find in Content Browser"), 
						FSlateIcon(), 
						FUIAction(FExecuteAction::CreateStatic(&StructViewer::Helpers::FindInContentBrowser, InStruct))
						);
				}
				else
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("StructViewerMenuOpenSourceCode", "Open Source Code..."), 
						LOCTEXT("StructViewerMenuOpenSourceCode_Tooltip", "Open the source file for this struct in the IDE."), 
						FSlateIcon(), 
						FUIAction(FExecuteAction::CreateStatic(&StructViewer::Helpers::OpenStructInIDE, InStruct))
						);
				}
				
				MenuBuilder.AddMenuEntry(
					LOCTEXT("StructViewerMenuCopyStructPath", "Copy Struct Path"),
					LOCTEXT("StructViewerMenuCopyStructPath_Tooltip", "Copies the path for this struct (e.g '/Script/Module.StructName') to the clipboard."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateStatic(&StructViewer::Helpers::CopyStructPathToClipboard, InStruct))
					);
			}

			return MenuBuilder.MakeWidget();
		}
	} // namespace Helpers
} // namespace StructViewer

/** Delegate used with the Struct Viewer in 'struct picking' mode. You'll bind a delegate when the struct viewer widget is created, which will be fired off when the selected struct is double clicked */
DECLARE_DELEGATE_OneParam(FOnStructItemDoubleClickDelegate, TSharedPtr<FStructViewerNode>);

/** The item used for visualizing the struct in the tree. */
class SStructItem : public STableRow<TSharedPtr<FString>>
{
public:

	SLATE_BEGIN_ARGS(SStructItem)
		: _StructDisplayName()
		, _bIsInStructViewer(true)
		, _bDynamicStructLoading(true)
		, _HighlightText()
		, _TextColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
	{}

	/** The struct name this item contains. */
	SLATE_ARGUMENT(FText, StructDisplayName)
	/** true if this item is in a Struct Viewer (as opposed to a Struct Picker) */
	SLATE_ARGUMENT(bool, bIsInStructViewer)
	/** true if this item should allow dynamic struct loading */
	SLATE_ARGUMENT(bool, bDynamicStructLoading)
	/** The text this item should highlight, if any. */
	SLATE_ARGUMENT(FText, HighlightText)
	/** The color text this item will use. */
	SLATE_ARGUMENT(FSlateColor, TextColor)
	/** The node this item is associated with. */
	SLATE_ARGUMENT(TSharedPtr<FStructViewerNode>, AssociatedNode)
	/** the delegate for handling double clicks outside of the SStructItem */
	SLATE_ARGUMENT(FOnStructItemDoubleClickDelegate, OnStructItemDoubleClicked)
	/** On Struct Picked callback. */
	SLATE_EVENT(FOnDragDetected, OnDragDetected)

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		StructDisplayName = InArgs._StructDisplayName;
		bIsInStructViewer = InArgs._bIsInStructViewer;
		bDynamicStructLoading = InArgs._bDynamicStructLoading;
		AssociatedNode = InArgs._AssociatedNode;
		OnDoubleClicked = InArgs._OnStructItemDoubleClicked;

		auto BuildToolTip = [this]() -> TSharedPtr<SToolTip>
		{
			TSharedPtr<SToolTip> ToolTip;

			TSharedPtr<IPropertyHandle> PropertyHandle = AssociatedNode->GetPropertyHandle();
			if (PropertyHandle && AssociatedNode->IsRestricted())
			{
				FText RestrictionToolTip;
				PropertyHandle->GenerateRestrictionToolTip(*AssociatedNode->GetStructName(), RestrictionToolTip);

				ToolTip = SNew(SToolTip).Text(RestrictionToolTip);
			}
			else if (!AssociatedNode->GetStructPath().IsNull())
			{
				const UScriptStruct* Struct = AssociatedNode->GetStruct();
				if (Struct != nullptr && Struct->GetBoolMetaDataHierarchical("ShowTooltip"))
				{
					const FText ToolTipText = FText::Format(LOCTEXT("ToolTipFormat", "{0}\n\n{1}"), 
						AssociatedNode->GetStruct()->GetToolTipText(), 
						FText::FromString(AssociatedNode->GetStructPath().ToString())
					);
					
					ToolTip = SNew(SToolTip).Text(ToolTipText);
				}
				else
				{
					ToolTip = SNew(SToolTip).Text(FText::FromString(AssociatedNode->GetStructPath().ToString()));	
				}
			}

			return ToolTip;
		};

		const bool bIsRestricted = AssociatedNode->IsRestricted();

		this->ChildSlot
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 6.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SStructItem::GetIconImage)
					.Visibility(this, &SStructItem::GetIconVisibility)
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f, 3.0f, 6.0f, 3.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(StructDisplayName)
					.HighlightText(InArgs._HighlightText)
					.ColorAndOpacity(this, &SStructItem::GetTextColor)
					.ToolTip(BuildToolTip())
					.IsEnabled(!bIsRestricted)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 6.0f, 0.0f)
				[
					SNew(SComboButton)
					.ContentPadding(FMargin(2.0f))
					.Visibility(this, &SStructItem::ShowOptions)
					.OnGetMenuContent(this, &SStructItem::GenerateDropDown)
				]
			];

		TextColor = InArgs._TextColor;

		UE_LOGF(LogEditorStructViewer, VeryVerbose, "STRUCT [%ls]", *StructDisplayName.ToString());

		STableRow<TSharedPtr<FString>>::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			.OnDragDetected(InArgs._OnDragDetected),
			InOwnerTableView
			);
	}

private:
	FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
	{
		// If in a Struct Viewer and it has not been loaded, load the struct when double-left clicking.
		if (bIsInStructViewer)
		{
			if (bDynamicStructLoading && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				AssociatedNode->LoadStruct();
			}

			// If there is a struct asset, open its asset editor; otherwise try to open the struct header
			if (const UUserDefinedStruct* UserDefinedStruct = AssociatedNode->GetStructAsset())
			{
				StructViewer::Helpers::OpenAssetEditor(UserDefinedStruct);
			}
			else if (const UScriptStruct* Struct = AssociatedNode->GetStruct())
			{
				StructViewer::Helpers::OpenStructInIDE(Struct);
			}
		}
		else
		{
			OnDoubleClicked.ExecuteIfBound(AssociatedNode);
		}

		return FReply::Handled();
	}

	EVisibility ShowOptions() const
	{
		// If it's in viewer mode, show the options combo button.
		return (bIsInStructViewer && AssociatedNode->GetStruct())
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}

	/**
	 * Generates the drop down menu for the item.
	 *
	 * @return		The drop down menu widget.
	 */
	TSharedRef<SWidget> GenerateDropDown()
	{
		if (const UScriptStruct* Struct = AssociatedNode->GetStruct())
		{
			return StructViewer::Helpers::CreateMenu(Struct);
		}

		return SNullWidget::NullWidget;
	}

	/** Returns the text color for the item based on if it is selected or not. */
	FSlateColor GetTextColor() const
	{
		const TSharedPtr<ITypedTableView<TSharedPtr<FString>>> OwnerWidget = OwnerTablePtr.Pin();
		const TSharedPtr<FString>* MyItem = OwnerWidget->Private_ItemFromWidget(this);
		const bool bIsSelected = OwnerWidget->Private_IsItemSelected(*MyItem);

		if (bIsSelected)
		{
			return FSlateColor::UseForeground();
		}

		return TextColor;
	}

	const FSlateBrush* GetIconImage() const
	{
		if (!AssociatedNode)
		{
			return nullptr;
		}

		return FSlateIconFinder::FindIconBrushForClass(AssociatedNode->GetStruct(), "ClassIcon.Object");
	}

	EVisibility GetIconVisibility() const
	{
		const UScriptStruct* Struct = AssociatedNode ? AssociatedNode->GetStruct() : nullptr;
		return Struct && Struct->GetBoolMetaDataHierarchical("ShowIcon") && GetIconImage()
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}

private:
	/** The struct name for which this item is associated with. */
	FText StructDisplayName;

	/** true if in a Struct Viewer (as opposed to a Struct Picker). */
	bool bIsInStructViewer;

	/** true if dynamic struct loading is permitted. */
	bool bDynamicStructLoading;

	/** The text color for this item. */
	FSlateColor TextColor;

	/** The Struct Viewer Node this item is associated with. */
	TSharedPtr<FStructViewerNode> AssociatedNode;

	/** the on Double Clicked delegate */
	FOnStructItemDoubleClickDelegate OnDoubleClicked;
};

FStructHierarchy::FStructHierarchy()
{
	// Register with the Asset Registry to be informed when it is done loading up files.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &FStructHierarchy::DirtyStructHierarchy);
	AssetRegistryModule.Get().OnAssetAdded().AddRaw(this, &FStructHierarchy::AddAsset);
	AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FStructHierarchy::RemoveAsset);

	// Register to have Populate called when doing a Reload.
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FStructHierarchy::OnReloadComplete);

	FModuleManager::Get().OnModulesChanged().AddRaw(this, &FStructHierarchy::OnModulesChanged);

	PopulateStructHierarchy();
}

FStructHierarchy::~FStructHierarchy()
{
	// Unregister with the Asset Registry to be informed when it is done loading up files.
	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}

		// Unregister to have Populate called when doing a Reload.
		FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);
	}

	FModuleManager::Get().OnModulesChanged().RemoveAll(this);
}

FStructHierarchy& FStructHierarchy::Get()
{
	if (!Singleton)
	{
		Singleton = MakeUnique<FStructHierarchy>();
		Singleton->PopulateStructHierarchy();
	}
	return *Singleton;
}

FStructHierarchy* FStructHierarchy::GetPtr()
{
	return Singleton.Get();
}

void FStructHierarchy::DestroyInstance()
{
	Singleton.Reset();
}

void FStructHierarchy::UpdateStructHierarchy()
{
	if (bRefreshStructHierarchy)
	{
		bRefreshStructHierarchy = false;
		PopulateStructHierarchy();
	}
}

void FStructHierarchy::DirtyStructHierarchy()
{
	bRefreshStructHierarchy = true;
}

void FStructHierarchy::PopulateStructHierarchy()
{
	FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingStructHierarchy", "Rebuilding Struct Hierarchy"));
	SlowTask.MakeDialog();

	// Make a dummy root node
	StructRootNode = MakeShared<FStructViewerNodeData>();

	// Add the tree of native structs
	{
		TMap<const UScriptStruct*, TSharedPtr<FStructViewerNodeData>> DataNodes;
		DataNodes.Add(nullptr, StructRootNode);

		TSet<const UScriptStruct*> Visited;
		UPackage* TransientPackage = GetTransientPackage();

		// Go through all of the structs and see if they should be added to the list.
		for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
		{
			const UScriptStruct* CurrentStruct = *StructIt;
			if (Visited.Contains(CurrentStruct) || CurrentStruct->GetOutermost() == TransientPackage)
			{
				// Skip transient structs as they are dead leftovers from user struct editing
				continue;
			}

			while (CurrentStruct)
			{
				const UScriptStruct* SuperStruct = Cast<UScriptStruct>(CurrentStruct->GetSuperStruct());

				TSharedPtr<FStructViewerNodeData>& ParentEntryRef = DataNodes.FindOrAdd(SuperStruct);
				if (!ParentEntryRef.IsValid())
				{
					check(SuperStruct); // The null entry should have been created above
					ParentEntryRef = MakeShared<FStructViewerNodeData>(SuperStruct);
				}

				// Need to have a pointer in-case the ref moves to avoid an extra look up in the map
				TSharedPtr<FStructViewerNodeData> ParentEntry = ParentEntryRef;

				TSharedPtr<FStructViewerNodeData>& MyEntryRef = DataNodes.FindOrAdd(CurrentStruct);
				if (!MyEntryRef.IsValid())
				{
					MyEntryRef = MakeShared<FStructViewerNodeData>(CurrentStruct);
				}

				// Need to re-acquire the reference in the struct as it may have been invalidated from a re-size
				if (!Visited.Contains(CurrentStruct))
				{
					ParentEntry->AddChild(MyEntryRef.ToSharedRef());
					Visited.Add(CurrentStruct);
				}

				CurrentStruct = SuperStruct;
			}
		}
	}

	// Add any struct assets directly under the root (since they don't support inheritance)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		FARFilter Filter;
		Filter.ClassPaths.Add(UUserDefinedStruct::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;

		TArray<FAssetData> UserDefinedStructsList;
		AssetRegistryModule.Get().GetAssets(Filter, UserDefinedStructsList);

		for (const FAssetData& UserDefinedStructData : UserDefinedStructsList)
		{
			if (!UserDefinedStructData.IsAssetLoaded())
			{
				// If the asset is loaded it was added by the object iterator
				StructRootNode->AddChild(MakeShared<FStructViewerNodeData>(UserDefinedStructData));
			}
		}
	}

	// All viewers must refresh.
	PopulateStructViewerDelegate.Broadcast();
}

TSharedPtr<FStructViewerNodeData> FStructHierarchy::FindNodeByStructPath(const TSharedRef<FStructViewerNodeData>& InRootNode, const FSoftObjectPath& InStructPath)
{
	// Check if the current node is the struct path that is being searched for
	if (InRootNode->GetStructPath() == InStructPath)
	{
		return InRootNode;
	}

	// Search the children recursively, one of them might have the node
	for (const TSharedPtr<FStructViewerNodeData>& ChildNode : InRootNode->GetChildNodes())
	{
		// Check the child, then check the return to see if it is valid. If it is valid, end the recursion.
		TSharedPtr<FStructViewerNodeData> ReturnNode = FindNodeByStructPath(ChildNode.ToSharedRef(), InStructPath);
		if (ReturnNode)
		{
			return ReturnNode;
		}
	}

	return nullptr;
}

bool FStructHierarchy::FindAndRemoveNodeByStructPath(const TSharedRef<FStructViewerNodeData>& InRootNode, const FSoftObjectPath& InStructPath)
{
	// Check if the current node contains a child of struct path that is being searched for
	if (InRootNode->RemoveChild(InStructPath))
	{
		return true;
	}

	// Search the children recursively, one of them might have the node
	for (const TSharedPtr<FStructViewerNodeData>& ChildNode : InRootNode->GetChildNodes())
	{
		// Check the child, then check the return to see if it is valid. If it is valid, end the recursion.
		if (FindAndRemoveNodeByStructPath(ChildNode.ToSharedRef(), InStructPath))
		{
			return true;
		}
	}

	return false;
}

void FStructHierarchy::AddAsset(const FAssetData& InAddedAssetData)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if (!AssetRegistryModule.Get().IsLoadingAssets())
	{
		// Only handle structs
		UClass* AssetClass = InAddedAssetData.GetClass();

		if (AssetClass && AssetClass->IsChildOf(UScriptStruct::StaticClass()))
		{
			// Make sure that the node does not already exist. There is a bit of double adding going on at times and this prevents it.
			if (!FindNodeByStructPath(InAddedAssetData.GetSoftObjectPath()))
			{
				// User defined structs are always root level structs
				StructRootNode->AddChild(MakeShared<FStructViewerNodeData>(InAddedAssetData));

				// All Viewers must repopulate.
				PopulateStructViewerDelegate.Broadcast();
			}
		}
	}
}

void FStructHierarchy::RemoveAsset(const FAssetData& InRemovedAssetData)
{
	if (FindAndRemoveNodeByStructPath(InRemovedAssetData.GetSoftObjectPath()))
	{
		// All viewers must refresh.
		PopulateStructViewerDelegate.Broadcast();
	}
}

void FStructHierarchy::OnReloadComplete(EReloadCompleteReason Reason)
{
	DirtyStructHierarchy();
}

void FStructHierarchy::OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange)
{
	if (ReasonForChange == EModuleChangeReason::ModuleLoaded || ReasonForChange == EModuleChangeReason::ModuleUnloaded)
	{
		DirtyStructHierarchy();
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SStructViewer::Construct(const FArguments& InArgs, const FStructViewerInitializationOptions& InInitOptions)
{
	bNeedsRefresh = true;
	NumStructs = 0;

	bCanShowInternalStructs = true;

	// Listen for when view settings are changed
	UStructViewerSettings::OnSettingChanged().AddSP(this, &SStructViewer::HandleSettingChanged);

	InitOptions = InInitOptions;

	OnStructPicked = InArgs._OnStructPickedDelegate;

	TextFilterPtr = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));

	bSaveExpansionStates = true;
	bPendingSetExpansionStates = false;

	bEnableStructDynamicLoading = InInitOptions.bEnableStructDynamicLoading;

	EVisibility HeaderVisibility = (this->InitOptions.Mode == EStructViewerMode::StructBrowsing) ? EVisibility::Visible : EVisibility::Collapsed;

	// Set these values to the user specified settings.
	bShowUnloadedStructs = InitOptions.bShowUnloadedStructs;
	bool bHasTitle = InitOptions.ViewerTitleString.IsEmpty() == false;

	// If set to default, decide what display mode to use.
	if (InitOptions.DisplayMode == EStructViewerDisplayMode::DefaultView)
	{
		// By default the Browser uses the tree view, the Picker the list. The option is available to users to force to another display mode when creating the Struct Browser/Picker.
		if (InitOptions.Mode == EStructViewerMode::StructBrowsing)
		{
			InitOptions.DisplayMode = EStructViewerDisplayMode::TreeView;
		}
		else
		{
			InitOptions.DisplayMode = EStructViewerDisplayMode::ListView;
		}
	}

	// Create the asset discovery indicator
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_Vertical);
	FOnContextMenuOpening OnContextMenuOpening;
	if (InitOptions.Mode == EStructViewerMode::StructBrowsing)
	{
		OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SStructViewer::BuildMenuWidget);
	}

	SAssignNew(StructList, SListView<TSharedPtr<FStructViewerNode>>)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(&RootTreeItems)
		// Generates the actual widget for a tree item
		.OnGenerateRow(this, &SStructViewer::OnGenerateRowForStructViewer)
		// Generates the right click menu.
		.OnContextMenuOpening(OnContextMenuOpening)
		// Find out when the user selects something in the tree
		.OnSelectionChanged(this, &SStructViewer::OnStructViewerSelectionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.Visibility(EVisibility::Collapsed)
			+SHeaderRow::Column(TEXT("Struct"))
			.DefaultLabel(NSLOCTEXT("StructViewer", "Struct", "Struct"))
		);

	SAssignNew(StructTree, STreeView<TSharedPtr<FStructViewerNode>>)
		.SelectionMode(ESelectionMode::Single)
		.TreeItemsSource(&RootTreeItems)
		// Called to child items for any given parent item
		.OnGetChildren(this, &SStructViewer::OnGetChildrenForStructViewerTree)
		// Called to handle recursively expanding/collapsing items
		.OnSetExpansionRecursive(this, &SStructViewer::SetAllExpansionStates_Helper)
		// Generates the actual widget for a tree item
		.OnGenerateRow(this, &SStructViewer::OnGenerateRowForStructViewer)
		// Generates the right click menu.
		.OnContextMenuOpening(OnContextMenuOpening)
		// Find out when the user selects something in the tree
		.OnSelectionChanged(this, &SStructViewer::OnStructViewerSelectionChanged)
		// Called when the expansion state of an item changes
		.OnExpansionChanged(this, &SStructViewer::OnStructViewerExpansionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.Visibility(EVisibility::Collapsed)
			+ SHeaderRow::Column(TEXT("Struct"))
			.DefaultLabel(NSLOCTEXT("StructViewer", "Struct", "Struct"))
		);

	TSharedRef<STreeView<TSharedPtr<FStructViewerNode>>> StructTreeView = StructTree.ToSharedRef();
	TSharedRef<SListView<TSharedPtr<FStructViewerNode>>> StructListView = StructList.ToSharedRef();

	// Holds the bulk of the struct viewer's sub-widgets, to be added to the widget after construction
	TSharedPtr<SWidget> StructViewerContent =
		SNew(SBox)
		.MaxDesiredHeight(800.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(InitOptions.bShowBackgroundBorder ? "ToolPanel.GroupBorder" : "NoBorder"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(bHasTitle ? EVisibility::Visible : EVisibility::Collapsed)
						.ColorAndOpacity(FAppStyle::GetColor("MultiboxHookColor"))
						.Text(InitOptions.ViewerTitleString)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(2.0f, 2.0f)
					[
						SAssignNew(SearchBox, SSearchBox)
						.OnTextChanged(this, &SStructViewer::OnFilterTextChanged)
						.OnTextCommitted(this, &SStructViewer::OnFilterTextCommitted)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.Visibility(HeaderVisibility)
				]

				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SOverlay)

					+SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SVerticalBox)

						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SScrollBorder, StructTreeView)
							.Visibility(InitOptions.DisplayMode == EStructViewerDisplayMode::TreeView ? EVisibility::Visible : EVisibility::Collapsed)
							[
								StructTreeView
							]
						]

						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SScrollBorder, StructListView)
							.Visibility(InitOptions.DisplayMode == EStructViewerDisplayMode::ListView ? EVisibility::Visible : EVisibility::Collapsed)
							[
								StructListView
							]
						]
					]

					+SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Bottom)
					.Padding(FMargin(24, 0, 24, 0))
					[
						// Asset discovery indicator
						AssetDiscoveryIndicator
					]
				]

				// Bottom panel
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					// Asset count
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					.Padding(8, 0)
					[
						SNew(STextBlock)
						.Text(this, &SStructViewer::GetStructCountText)
					]

					// View mode combo button
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(ViewOptionsComboButton, SComboButton)
						.ContentPadding(0)
						.ForegroundColor(this, &SStructViewer::GetViewButtonForegroundColor)
						.ButtonStyle(FAppStyle::Get(), "ToggleButton") // Use the tool bar item style for this button
						.OnGetMenuContent(this, &SStructViewer::GetViewButtonContent)
						.ButtonContent()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage).Image(FAppStyle::GetBrush("GenericViewButton"))
							]

							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2, 0, 0, 0)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock).Text(LOCTEXT("ViewButton", "View Options"))
							]
						]
					]
				]
			]
		];

	if (ViewOptionsComboButton.IsValid())
	{
		ViewOptionsComboButton->SetVisibility(InitOptions.bAllowViewOptions ? EVisibility::Visible : EVisibility::Collapsed);
	}

	// When using a struct picker in list-view mode, the widget will auto-focus the search box
	// and allow the up and down arrow keys to navigate and enter to pick without using the mouse ever
	if (InitOptions.Mode == EStructViewerMode::StructPicker && InitOptions.DisplayMode == EStructViewerDisplayMode::ListView)
	{
		this->ChildSlot
			[
				SNew(SListViewSelectorDropdownMenu<TSharedPtr<FStructViewerNode>>, SearchBox, StructList)
				[
					StructViewerContent.ToSharedRef()
				]
			];
	}
	else
	{
		this->ChildSlot
			[
				StructViewerContent.ToSharedRef()
			];
	}

	// Ensure the struct hierarchy exists, and and watch for changes
	FStructHierarchy::Get().GetPopulateStructViewerDelegate().AddSP(this, &SStructViewer::Refresh);

	// Request delayed setting of focus to the search box
	bPendingFocusNextFrame = true;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> SStructViewer::GetContent()
{
	return SharedThis(this);
}

SStructViewer::~SStructViewer()
{
	// No longer watching for changes
	if (FStructHierarchy* StructHierarchy = FStructHierarchy::GetPtr())
	{
		StructHierarchy->GetPopulateStructViewerDelegate().RemoveAll(this);
	}

	// Remove the listener for when view settings are changed
	UStructViewerSettings::OnSettingChanged().RemoveAll(this);
}

void SStructViewer::ClearSelection()
{
	StructTree->ClearSelection();
}

void SStructViewer::OnGetChildrenForStructViewerTree(TSharedPtr<FStructViewerNode> InParent, TArray<TSharedPtr<FStructViewerNode>>& OutChildren)
{
	// Simply return the children, it's already setup.
	OutChildren = InParent->GetChildNodes();
}

void SStructViewer::OnStructViewerSelectionChanged(TSharedPtr<FStructViewerNode> Item, ESelectInfo::Type SelectInfo)
{
	// Do not act on selection change when it is for navigation
	if (SelectInfo == ESelectInfo::OnNavigation && InitOptions.DisplayMode == EStructViewerDisplayMode::ListView)
	{
		return;
	}

	// Sometimes the item is not valid anymore due to filtering.
	if (Item.IsValid() == false || Item->IsRestricted())
	{
		return;
	}

	if (InitOptions.Mode != EStructViewerMode::StructBrowsing)
	{
		// Attempt to ensure the struct is loaded
		Item->LoadStruct();

		// Check if the item passed the filter, parent items might be displayed but filtered out and thus not desired to be selected.
		if (Item->PassedFilter())
		{
			OnStructPicked.ExecuteIfBound(Item->GetStruct());
		}
		else
		{
			OnStructPicked.ExecuteIfBound(nullptr);
		}
	}
}

void SStructViewer::OnStructViewerExpansionChanged(TSharedPtr<FStructViewerNode> Item, bool bExpanded)
{
	// Sometimes the item is not valid anymore due to filtering.
	if (!Item.IsValid() || Item->IsRestricted())
	{
		return;
	}

	ExpansionStateMap.Add(Item->GetStructPath(), bExpanded);
}

TSharedPtr<SWidget> SStructViewer::BuildMenuWidget()
{
	TArray<TSharedPtr<FStructViewerNode>> SelectedList;

	// Based upon which mode the viewer is in, pull the selected item.
	if (InitOptions.DisplayMode == EStructViewerDisplayMode::TreeView)
	{
		SelectedList = StructTree->GetSelectedItems();
	}
	else
	{
		SelectedList = StructList->GetSelectedItems();
	}

	// If there is no selected item, return a null widget.
	if (SelectedList.Num() == 0)
	{
		return SNullWidget::NullWidget;
	}

	// Get the struct
	const UScriptStruct* RightClickStruct = SelectedList[0]->GetStruct();
	if (bEnableStructDynamicLoading && !RightClickStruct)
	{
		SelectedList[0]->LoadStruct();
		RightClickStruct = SelectedList[0]->GetStruct();

		// Populate the tree/list so any changes to previously unloaded structs will be reflected.
		Refresh();
	}

	return StructViewer::Helpers::CreateMenu(RightClickStruct);
}

TSharedRef<ITableRow> SStructViewer::OnGenerateRowForStructViewer(TSharedPtr<FStructViewerNode> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	// If the item was accepted by the filter, leave it bright, otherwise dim it.
	float AlphaValue = Item->PassedFilter() ? 1.0f : 0.5f;
	TSharedRef<SStructItem> ReturnRow = SNew(SStructItem, OwnerTable)
		.StructDisplayName(Item->GetStructDisplayName(InitOptions.NameTypeToDisplay))
		.HighlightText(SearchBox->GetText())
		.TextColor(FLinearColor(1.0f, 1.0f, 1.0f, AlphaValue))
		.AssociatedNode(Item)
		.bIsInStructViewer(InitOptions.Mode == EStructViewerMode::StructBrowsing)
		.bDynamicStructLoading(bEnableStructDynamicLoading)
		.OnDragDetected(this, &SStructViewer::OnDragDetected)
		.OnStructItemDoubleClicked(FOnStructItemDoubleClickDelegate::CreateSP(this, &SStructViewer::ToggleExpansionState_Helper));

	// Expand the item if needed.
	if (!bPendingSetExpansionStates)
	{
		bool* bIsExpanded = ExpansionStateMap.Find(Item->GetStructPath());
		if (bIsExpanded && *bIsExpanded)
		{
			bPendingSetExpansionStates = true;
		}
	}

	return ReturnRow;
}

const TArray<TSharedPtr<FStructViewerNode>> SStructViewer::GetSelectedItems() const
{
	if (InitOptions.DisplayMode == EStructViewerDisplayMode::ListView)
	{
		return StructList->GetSelectedItems();
	}
	return StructTree->GetSelectedItems();
}

const int SStructViewer::GetNumItems() const
{
	return NumStructs;
}

FSlateColor SStructViewer::GetViewButtonForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");
	static const FName DefaultForegroundName("DefaultForeground");

	return ViewOptionsComboButton->IsHovered() ? FAppStyle::GetSlateColor(InvertedForegroundName) : FAppStyle::GetSlateColor(DefaultForegroundName);
}

TSharedRef<SWidget> SStructViewer::GetViewButtonContent()
{
	// Get all menu extenders for this context menu from the content browser module

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr, nullptr, /*bCloseSelfOnly=*/ true);

	MenuBuilder.AddMenuEntry(LOCTEXT("ExpandAll", "Expand All"), LOCTEXT("ExpandAll_Tooltip", "Expands the entire tree"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SStructViewer::SetAllExpansionStates, bool(true))), NAME_None, EUserInterfaceActionType::Button);
	MenuBuilder.AddMenuEntry(LOCTEXT("CollapseAll", "Collapse All"), LOCTEXT("CollapseAll_Tooltip", "Collapses the entire tree"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SStructViewer::SetAllExpansionStates, bool(false))), NAME_None, EUserInterfaceActionType::Button);

	MenuBuilder.BeginSection("Filters", LOCTEXT("StructViewerFiltersHeading", "Struct Filters"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowInternalStructsOption", "Show Internal Structs"),
			LOCTEXT("ShowInternalStructsOptionToolTip", "Shows internal-use only structs in the view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SStructViewer::ToggleShowInternalStructs),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SStructViewer::IsShowingInternalStructs)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("DeveloperViewType", LOCTEXT("DeveloperViewTypeHeading", "Developer Folder Filter"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NoneDeveloperViewOption", "None"),
			LOCTEXT("NoneDeveloperViewOptionToolTip", "Filter structs to show no structs in developer folders."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SStructViewer::SetCurrentDeveloperViewType, EStructViewerDeveloperType::SVDT_None),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SStructViewer::IsCurrentDeveloperViewType, EStructViewerDeveloperType::SVDT_None)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CurrentUserDeveloperViewOption", "Current Developer"),
			LOCTEXT("CurrentUserDeveloperViewOptionToolTip", "Filter structs to allow structs in the current user's development folder."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SStructViewer::SetCurrentDeveloperViewType, EStructViewerDeveloperType::SVDT_CurrentUser),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SStructViewer::IsCurrentDeveloperViewType, EStructViewerDeveloperType::SVDT_CurrentUser)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AllUsersDeveloperViewOption", "All Developers"),
			LOCTEXT("AllUsersDeveloperViewOptionToolTip", "Filter structs to allow structs in all users' development folders."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SStructViewer::SetCurrentDeveloperViewType, EStructViewerDeveloperType::SVDT_All),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SStructViewer::IsCurrentDeveloperViewType, EStructViewerDeveloperType::SVDT_All)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SStructViewer::SetCurrentDeveloperViewType(EStructViewerDeveloperType NewType)
{
	if (ensure((int)NewType < (int)EStructViewerDeveloperType::SVDT_Max) && NewType != GetDefault<UStructViewerSettings>()->DeveloperFolderType)
	{
		GetMutableDefault<UStructViewerSettings>()->DeveloperFolderType = NewType;
		GetMutableDefault<UStructViewerSettings>()->PostEditChange();
	}
}

EStructViewerDeveloperType SStructViewer::GetCurrentDeveloperViewType() const
{
	if (!InitOptions.bAllowViewOptions)
	{
		return EStructViewerDeveloperType::SVDT_All;
	}

	return GetDefault<UStructViewerSettings>()->DeveloperFolderType;
}

bool SStructViewer::IsCurrentDeveloperViewType(EStructViewerDeveloperType ViewType) const
{
	return GetCurrentDeveloperViewType() == ViewType;
}

void SStructViewer::GetInternalOnlyStructs(TArray<TSoftObjectPtr<const UScriptStruct>>& Structs)
{
	if (!InitOptions.bAllowViewOptions)
	{
		return;
	}
	Structs = GetDefault<UStructViewerProjectSettings>()->InternalOnlyStructs;
}

void SStructViewer::GetInternalOnlyPaths(TArray<FDirectoryPath>& Paths)
{
	if (!InitOptions.bAllowViewOptions)
	{
		return;
	}
	Paths = GetDefault<UStructViewerProjectSettings>()->InternalOnlyPaths;
}

FText SStructViewer::GetStructCountText() const
{
	const int32 NumSelectedStructs = GetSelectedItems().Num();
	if (NumSelectedStructs == 0)
	{
		return FText::Format(LOCTEXT("StructCountLabel", "{0} {0}|plural(one=item,other=items)"), NumStructs);
	}
	else
	{
		return FText::Format(LOCTEXT("StructCountLabelPlusSelection", "{0} {0}|plural(one=item,other=items) ({1} selected)"), NumStructs, NumSelectedStructs);
	}
}

void SStructViewer::ExpandRootNodes()
{
	for (int32 NodeIdx = 0; NodeIdx < RootTreeItems.Num(); ++NodeIdx)
	{
		ExpansionStateMap.Add(RootTreeItems[NodeIdx]->GetStructPath(), true);
		StructTree->SetItemExpansion(RootTreeItems[NodeIdx], true);
	}
}

FReply SStructViewer::OnDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	if (InitOptions.Mode == EStructViewerMode::StructBrowsing)
	{
		const TArray<TSharedPtr<FStructViewerNode>> SelectedItems = GetSelectedItems();

		if (SelectedItems.Num() > 0 && SelectedItems[0].IsValid())
		{
			TSharedRef<FStructViewerNode> Item = SelectedItems[0].ToSharedRef();

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			// Spawn a loaded user defined struct just like any other asset from the Content Browser.
			const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(Item->GetStructPath());
			if (AssetData.IsValid())
			{
				return FReply::Handled().BeginDragDrop(FContentBrowserDataDragDropOp::Legacy_New(MakeArrayView(&AssetData, 1)));
			}
		}
	}

	return FReply::Unhandled();
}

void SStructViewer::OnFilterTextChanged(const FText& InFilterText)
{
	// Update the compiled filter and report any syntax error information back to the user
	TextFilterPtr->SetFilterText(InFilterText);
	SearchBox->SetError(TextFilterPtr->GetFilterErrorText());

	// Repopulate the list to show only what has not been filtered out.
	Refresh();
}

void SStructViewer::OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (InitOptions.Mode == EStructViewerMode::StructPicker)
		{
			TArray<TSharedPtr<FStructViewerNode>> SelectedList = StructList->GetSelectedItems();
			if (SelectedList.Num() > 0)
			{
				TSharedPtr<FStructViewerNode> FirstSelected = SelectedList[0];
				const UScriptStruct* Struct = FirstSelected->GetStruct();

				// Try and ensure the struct is loaded
				if (bEnableStructDynamicLoading && !Struct)
				{
					FirstSelected->LoadStruct();
					Struct = FirstSelected->GetStruct();
				}

				// Check if the item passes the filter, parent items might be displayed but filtered out and thus not desired to be selected.
				if (Struct && FirstSelected->PassedFilter())
				{
					OnStructPicked.ExecuteIfBound(Struct);
				}
			}
		}
	}
}

void SStructViewer::SetAllExpansionStates(bool bInExpansionState)
{
	// Go through all the items in the root of the tree and recursively visit their children to set every item in the tree.
	for (int32 ChildIndex = 0; ChildIndex < RootTreeItems.Num(); ChildIndex++)
	{
		SetAllExpansionStates_Helper(RootTreeItems[ChildIndex], bInExpansionState);
	}
}

void SStructViewer::SetAllExpansionStates_Helper(TSharedPtr<FStructViewerNode> InNode, bool bInExpansionState)
{
	StructTree->SetItemExpansion(InNode, bInExpansionState);

	// Recursively go through the children.
	for (const TSharedPtr<FStructViewerNode>& ChildNode : InNode->GetChildNodes())
	{
		SetAllExpansionStates_Helper(ChildNode, bInExpansionState);
	}
}

void SStructViewer::ToggleExpansionState_Helper(TSharedPtr<FStructViewerNode> InNode)
{
	const bool bExpanded = StructTree->IsItemExpanded(InNode);
	StructTree->SetItemExpansion(InNode, !bExpanded);
}

bool SStructViewer::ExpandFilteredInNodes(TSharedPtr<FStructViewerNode> InNode)
{
	bool bShouldExpand = InNode->PassedFilter();

	for (const TSharedPtr<FStructViewerNode>& ChildNode : InNode->GetChildNodes())
	{
		bShouldExpand |= ExpandFilteredInNodes(ChildNode);
	}

	if (bShouldExpand)
	{
		StructTree->SetItemExpansion(InNode, true);
	}

	return bShouldExpand;
}

void SStructViewer::MapExpansionStatesInTree(TSharedPtr<FStructViewerNode> InItem)
{
	ExpansionStateMap.Add(InItem->GetStructPath(), StructTree->IsItemExpanded(InItem));

	// Map out all the children, this will be done recursively.
	for (const TSharedPtr<FStructViewerNode>& ChildItem : InItem->GetChildNodes())
	{
		MapExpansionStatesInTree(ChildItem);
	}
}

void SStructViewer::SetExpansionStatesInTree(TSharedPtr<FStructViewerNode> InItem)
{
	const bool* bIsExpanded = ExpansionStateMap.Find(InItem->GetStructPath());
	if (bIsExpanded)
	{
		StructTree->SetItemExpansion(InItem, *bIsExpanded);

		// No reason to set expansion states if the parent is not expanded, it does not seem to do anything.
		if (*bIsExpanded)
		{
			for (const TSharedPtr<FStructViewerNode>& ChildItem : InItem->GetChildNodes())
			{
				SetExpansionStatesInTree(ChildItem);
			}
		}
	}
	else
	{
		// Default to no expansion.
		StructTree->SetItemExpansion(InItem, false);
	}
}

int32 SStructViewer::CountTreeItems(FStructViewerNode* Node)
{
	if (Node == nullptr)
	{
		return 0;
	}

	int32 Count = 1;
	for (const TSharedPtr<FStructViewerNode>& ChildNode : Node->GetChildNodes())
	{
		Count += CountTreeItems(ChildNode.Get());
	}
	return Count;
}

void SStructViewer::Populate()
{
	bPendingSetExpansionStates = false;

	// If showing a struct tree, we may need to save expansion states.
	if (InitOptions.DisplayMode == EStructViewerDisplayMode::TreeView)
	{
		if (bSaveExpansionStates)
		{
			for (const TSharedPtr<FStructViewerNode>& ChildNode : RootTreeItems)
			{
				// Check if the item is actually expanded or if it's only expanded because it is root level.
				bool* bIsExpanded = ExpansionStateMap.Find(ChildNode->GetStructPath());
				if ((bIsExpanded && !*bIsExpanded) || !bIsExpanded)
				{
					StructTree->SetItemExpansion(ChildNode, false);
				}

				// Recursively map out the expansion state of the tree-node.
				MapExpansionStatesInTree(ChildNode);
			}
		}

		// This is set to false before the call to populate when it is not desired.
		bSaveExpansionStates = true;
	}

	// Empty the tree out so it can be redone.
	RootTreeItems.Empty();

	const bool ShowingInternalStructs = IsShowingInternalStructs();

	TArray<const UScriptStruct*> InternalStructs;
	TArray<FDirectoryPath> InternalPaths;

	// If we aren't showing the internal structs, then we need to know what structs to consider Internal Only, so let's gather them up from the settings object.
	if (!ShowingInternalStructs)
	{
		TArray<TSoftObjectPtr<const UScriptStruct>> InternalStructNames;
		GetInternalOnlyPaths(InternalPaths);
		GetInternalOnlyStructs(InternalStructNames);

		// Take the package names for the internal only structs and convert them into their UScriptStructs
		for (const TSoftObjectPtr<const UScriptStruct>& InternalStructName : InternalStructNames)
		{
			const TSharedPtr<FStructViewerNodeData> StructNode = FStructHierarchy::Get().FindNodeByStructPath(InternalStructName.ToSoftObjectPath());
			if (StructNode.IsValid())
			{
				if (const UScriptStruct* Struct = StructNode->GetStruct())
				{
					InternalStructs.Add(Struct);
				}
			}
		}
	}

	// Based on if the list or tree is visible we create what will be displayed differently.
	if (InitOptions.DisplayMode == EStructViewerDisplayMode::TreeView)
	{
		// The root node for the tree, will be dummy instance which we will skip.
		TSharedPtr<FStructViewerNode> RootNode;

		// Get the struct tree, passing in certain filter options.
		StructViewer::Helpers::GetStructTree(InitOptions, RootNode, *TextFilterPtr, bShowUnloadedStructs, GetCurrentDeveloperViewType(), ShowingInternalStructs, InternalStructs, InternalPaths);

		// Sort the tree
		RootNode->SortChildrenRecursive();

		// Check if we will restore expansion states, we will not if there is filtering happening.
		const bool bRestoreExpansionState = TextFilterPtr->GetFilterType() == ETextFilterExpressionType::Empty;

		// Add all the children of the dummy root.
		for (const TSharedPtr<FStructViewerNode>& ChildItem : RootNode->GetChildNodes())
		{
			RootTreeItems.Add(ChildItem);
			if (bRestoreExpansionState)
			{
				SetExpansionStatesInTree(ChildItem);
			}

			// Expand any items that pass the filter.
			if (TextFilterPtr->GetFilterType() != ETextFilterExpressionType::Empty)
			{
				ExpandFilteredInNodes(ChildItem);
			}
		}

		// Only display this option if the user wants it and in Picker Mode.
		if (InitOptions.bShowNoneOption && InitOptions.Mode == EStructViewerMode::StructPicker)
		{
			// @todo - It would seem smart to add this in before the other items, since it needs to be on top. However, that causes strange issues with saving/restoring expansion states. 
			// This is likely not very efficient since the list can have hundreds and even thousands of items.
			RootTreeItems.Insert(MakeShared<FStructViewerNode>(), 0);
		}

		NumStructs = 0;
		for (const TSharedPtr<FStructViewerNode>& ChildNode : RootTreeItems)
		{
			NumStructs += CountTreeItems(ChildNode.Get());
		}

		// Now that new items are in the tree, we need to request a refresh.
		StructTree->RequestTreeRefresh();
	}
	else
	{
		// Get the struct list, passing in certain filter options.
		StructViewer::Helpers::GetStructList(InitOptions, RootTreeItems, *TextFilterPtr, bShowUnloadedStructs, GetCurrentDeveloperViewType(), ShowingInternalStructs, InternalStructs, InternalPaths);

		// Sort the list alphabetically.
		FStructViewerNode::SortNodes(RootTreeItems, InitOptions.PropertyHandle);

		// Scroll to selected struct
		for (const TSharedPtr<FStructViewerNode>& Node : RootTreeItems)
		{
			if (Node.IsValid())
			{
				if (InitOptions.SelectedStruct == Node->GetStruct())
				{
					StructList->RequestScrollIntoView(Node);
				}
			}
		}

		// Only display this option if the user wants it and in Picker Mode.
		if (InitOptions.bShowNoneOption && InitOptions.Mode == EStructViewerMode::StructPicker)
		{
			TSharedRef<FStructViewerNode> NoneNode = MakeShared<FStructViewerNode>();

			// @todo - It would seem smart to add this in before the other items, since it needs to be on top. However, that causes strange issues with saving/restoring expansion states. 
			// This is likely not very efficient since the list can have hundreds and even thousands of items.
			if (StructViewer::Helpers::PassesFilter(NoneNode->GetStructName(), *TextFilterPtr) ||
				StructViewer::Helpers::PassesFilter(NoneNode->GetStructDisplayName().ToString(), *TextFilterPtr))
			{
				RootTreeItems.Insert(MoveTemp(NoneNode), 0);
			}
		}

		NumStructs = 0;
		for (const TSharedPtr<FStructViewerNode>& ChildNode : RootTreeItems)
		{
			NumStructs += CountTreeItems(ChildNode.Get());
		}

		// Now that new items are in the list, we need to request a refresh.
		StructList->RequestListRefresh();
	}
}

FReply SStructViewer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Forward key down to struct tree
	return StructTree->OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SStructViewer::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	if (RootTreeItems.Num() > 0)
	{
		StructTree->SetItemSelection(RootTreeItems[0], true, ESelectInfo::OnMouseClick);
		StructTree->SetItemExpansion(RootTreeItems[0], true);
		OnStructViewerSelectionChanged(RootTreeItems[0], ESelectInfo::OnMouseClick);
	}

	FSlateApplication::Get().SetKeyboardFocus(SearchBox.ToSharedRef(), EFocusCause::SetDirectly);

	return FReply::Unhandled();
}

bool SStructViewer::SupportsKeyboardFocus() const
{
	return true;
}

void SStructViewer::DestroyStructHierarchy()
{
	FStructHierarchy::DestroyInstance();
}

void SStructViewer::Refresh()
{
	bNeedsRefresh = true;
}

void SStructViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Will populate the struct hierarchy as needed.
	FStructHierarchy::Get().UpdateStructHierarchy();

	// Move focus to search box
	if (bPendingFocusNextFrame && SearchBox.IsValid())
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchBox.ToSharedRef(), WidgetToFocusPath);
		FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPath, EFocusCause::SetDirectly);
		bPendingFocusNextFrame = false;
	}

	if (bNeedsRefresh)
	{
		bNeedsRefresh = false;
		Populate();

		if (InitOptions.bExpandRootNodes)
		{
			ExpandRootNodes();
		}
	}

	if (bPendingSetExpansionStates)
	{
		check(RootTreeItems.Num() > 0);
		SetExpansionStatesInTree(RootTreeItems[0]);
		bPendingSetExpansionStates = false;
	}
}

bool SStructViewer::IsStructAllowed(const UScriptStruct* InStruct) const
{
	return StructViewer::Helpers::IsStructAllowed(InitOptions, InStruct);
}

void SStructViewer::HandleSettingChanged(FName PropertyName)
{
	if ((PropertyName == "DisplayInternalClasses") ||
		(PropertyName == "DeveloperFolderType") ||
		(PropertyName == NAME_None))	// @todo: Needed if PostEditChange was called manually, for now
	{
		Refresh();
	}
}

void SStructViewer::ToggleShowInternalStructs()
{
	check(IsToggleShowInternalStructsAllowed());
	GetMutableDefault<UStructViewerSettings>()->DisplayInternalStructs = !GetDefault<UStructViewerSettings>()->DisplayInternalStructs;
	GetMutableDefault<UStructViewerSettings>()->PostEditChange();
}

bool SStructViewer::IsToggleShowInternalStructsAllowed() const
{
	return bCanShowInternalStructs;
}

bool SStructViewer::IsShowingInternalStructs() const
{
	if (!InitOptions.bAllowViewOptions)
	{
		return true;
	}
	return IsToggleShowInternalStructsAllowed() && GetDefault<UStructViewerSettings>()->DisplayInternalStructs;
}

#undef LOCTEXT_NAMESPACE
