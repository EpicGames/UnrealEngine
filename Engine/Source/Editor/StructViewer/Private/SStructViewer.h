// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "StructViewerModule.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Settings/StructViewerSettings.h"
#include "Engine/EngineTypes.h"
#include "UObject/SoftObjectPath.h"

class FMenuBuilder;
class FTextFilterExpressionEvaluator;
class UBlueprint;
class SComboButton;
class FStructViewerNode;
class FTextFilterExpressionEvaluator;

class SStructViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStructViewer)
	{
	}

	SLATE_ARGUMENT(FOnStructPicked, OnStructPickedDelegate)

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param	InArgs			A declaration from which to construct the widget
	 * @param	InitOptions		Programmer-driven initialization options for this widget
	 */
	void Construct(const FArguments& InArgs, const FStructViewerInitializationOptions& InInitOptions);

	/** Gets the widget contents of the app */
	virtual TSharedRef<SWidget> GetContent();

	virtual ~SStructViewer();

	/** Empty the selection set. */
	virtual void ClearSelection();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	virtual bool SupportsKeyboardFocus() const override;

	/** Test to see whether the given struct would be allowed by this struct viewer */
	virtual bool IsStructAllowed(const UScriptStruct* InStruct) const;

	/** Destroys the internal Struct Hierarchy database */
	static void DestroyStructHierarchy();

private:
	/** Retrieves the children for the input node.
	 *	@param InParent				The parent node to retrieve the children from.
	 *	@param OutChildren			List of children for the parent node.
	 *
	 */
	void OnGetChildrenForStructViewerTree(TSharedPtr<FStructViewerNode> InParent, TArray<TSharedPtr<FStructViewerNode>>& OutChildren);

	/** Creates the row widget when called by Slate when an item appears on the tree. */
	TSharedRef<ITableRow> OnGenerateRowForStructViewer(TSharedPtr<FStructViewerNode> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Invoked when the user attempts to drag an item out of the struct browser */
	FReply OnDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent) override;

	/** Called by Slate when the filter box changes text. */
	void OnFilterTextChanged(const FText& InFilterText);

	/** Called when enter is hit in search box */
	void OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Called by Slate when an item is selected from the tree/list. */
	void OnStructViewerSelectionChanged(TSharedPtr<FStructViewerNode> Item, ESelectInfo::Type SelectInfo);

	/** Called by Slate when an item is expanded/collapsed from the tree/list. */
	void OnStructViewerExpansionChanged(TSharedPtr<FStructViewerNode> Item, bool bExpanded);

	/**
	 *	Sets all expansion states in the tree.
	 *
	 *	@param bInExpansionState			The expansion state to set the tree to.
	 */
	void SetAllExpansionStates(bool bInExpansionState);

	/**
	 *	A helper function to recursively set the tree.
	 *
	 *	@param	InNode						The current node in the tree.
	 *	@param	bInExpansionState			The expansion state to set the tree to.
	 */
	void SetAllExpansionStates_Helper(TSharedPtr<FStructViewerNode> InNode, bool bInExpansionState);

	/**
	 *	A helper function to toggle expansion state of a single node
	 *
	 *	@param	InNode						The node to toggle expansion.
	 */
	void ToggleExpansionState_Helper(TSharedPtr<FStructViewerNode> InNode);

	/** Builds the right click menu widget for the selected node. */
	TSharedPtr<SWidget> BuildMenuWidget();

	/** Recursive function to expand nodes not filtered out of the tree
	*	@param	InNode				The current node to inspect for expansion.
	*
	*	@return bool				true if the child expanded, thus the parent should.
	*/
	bool ExpandFilteredInNodes(TSharedPtr<FStructViewerNode> InNode);

	/** Recursive function to map the expansion states of items in the tree.
	 *	@param InItem		The current item to examine the expansion state of.
	 */
	void MapExpansionStatesInTree(TSharedPtr<FStructViewerNode> InItem);

	/** Recursive function to set the expansion states of items in the tree.
	 *	@param InItem		The current item to set the expansion state of.
	 */
	void SetExpansionStatesInTree(TSharedPtr<FStructViewerNode> InItem);

	/** Sends a requests to the Struct Viewer to refresh itself the next chance it gets */
	void Refresh();

	/** Populates the tree with items based on the current filter. */
	void Populate();

	/** Returns an array of the currently selected StructViewerNodes */
	const TArray<TSharedPtr<FStructViewerNode>> GetSelectedItems() const;

	/** Expands all of the root nodes */
	virtual void ExpandRootNodes();

	/** Returns the foreground color for the view button */
	FSlateColor GetViewButtonForegroundColor() const;

	/** Handler for when the view combo button is clicked */
	TSharedRef<SWidget> GetViewButtonContent();

	/** Gets the text for the struct count label */
	FText GetStructCountText() const;

	/** Sets the view type and updates lists accordingly */
	void SetCurrentDeveloperViewType(EStructViewerDeveloperType NewType);

	/** Gets the current view type (list or tile) */
	EStructViewerDeveloperType GetCurrentDeveloperViewType() const;

	/** Returns true if ViewType is the current view type */
	bool IsCurrentDeveloperViewType(EStructViewerDeveloperType ViewType) const;

	/** Toggle whether internal use structs should be shown or not */
	void ToggleShowInternalStructs();

	/** Whether or not it's possible to show internal use structs */
	bool IsShowingInternalStructs() const;

	/** Whether or not it's possible to show internal use structs */
	bool IsToggleShowInternalStructsAllowed() const;

	/** Get the total number of structs passing the current filters.*/
	const int GetNumItems() const;

	/** Count the number of tree items in the specified hierarchy*/
	int32 CountTreeItems(FStructViewerNode* Node);

	/** Handle the settings for StructViewer changing.*/
	void HandleSettingChanged(FName PropertyName);

	/** Accessor for the struct names that have been marked as internal only in settings */
	void GetInternalOnlyStructs(TArray<TSoftObjectPtr<const UScriptStruct>>& Structs);

	/** Accessor for the struct paths that have been marked as internal only in settings */
	void GetInternalOnlyPaths(TArray<FDirectoryPath>& Paths);

private:
	/** Init options, cached */
	FStructViewerInitializationOptions InitOptions;

	/** The items to be displayed in the tree. */
	TArray<TSharedPtr<FStructViewerNode>> RootTreeItems;

	/** Compiled filter search terms. */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilterPtr;

	/** Holds the Slate Tree widget which holds the structs for the Struct Viewer. */
	TSharedPtr<STreeView<TSharedPtr<FStructViewerNode>>> StructTree;

	/** Holds the Slate List widget which holds the structs for the Struct Viewer. */
	TSharedPtr<SListView<TSharedPtr<FStructViewerNode>>> StructList;

	/** The struct Search Box, used for filtering the structs visible. */
	TSharedPtr<SSearchBox> SearchBox;

	/** true to filter for unloaded structs. */
	bool bShowUnloadedStructs;

	/** true to allow struct dynamic loading. */
	bool bEnableStructDynamicLoading;

	/** Callback that's fired when a struct is selected while in 'struct picking' mode */
	FOnStructPicked OnStructPicked;

	/** true if expansions states should be saved when compiling. */
	bool bSaveExpansionStates;

	/** The map holding the expansion state map for the tree. */
	TMap<FSoftObjectPath, bool> ExpansionStateMap;

	/** True if the Struct Viewer needs to be repopulated at the next appropriate opportunity, occurs whenever structs are added, removed, renamed, etc. */
	bool bNeedsRefresh;

	/** True if the search box will take keyboard focus next frame */
	bool bPendingFocusNextFrame;

	/** True if we need to set the tree expansion states according to our local copy next tick */
	bool bPendingSetExpansionStates;

	/** Indicates if the 'Show Internal Structs' option should be enabled or disabled */
	bool bCanShowInternalStructs;

	/** The button that displays view options */
	TSharedPtr<SComboButton> ViewOptionsComboButton;

	/** Number of structs that passed the filter*/
	int32 NumStructs;
};
