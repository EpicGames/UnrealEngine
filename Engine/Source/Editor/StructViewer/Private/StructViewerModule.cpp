// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructViewerModule.h"

#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "SStructViewer.h"
#include "StructViewerProjectSettings.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "StructViewer"

IMPLEMENT_MODULE(FStructViewerModule, StructViewer);

namespace StructViewerModule
{
	static const FName StructViewerApp = FName("StructViewerApp");
}

TSharedRef<SDockTab> CreateStructPickerTab(const FSpawnTabArgs& Args)
{
	FStructViewerInitializationOptions InitOptions;
	InitOptions.Mode = EStructViewerMode::StructBrowsing;
	InitOptions.DisplayMode = EStructViewerDisplayMode::TreeView;

	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SStructViewer, InitOptions)
			.OnStructPickedDelegate(FOnStructPicked())
		];
}


void FStructViewerModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(StructViewerModule::StructViewerApp, FOnSpawnTab::CreateStatic(&CreateStructPickerTab))
		.SetDisplayName(NSLOCTEXT("StructViewerApp", "TabTitle", "Struct Viewer"))
		.SetTooltipText(NSLOCTEXT("StructViewerApp", "TooltipText", "Displays all structs that exist within this project."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.UserDefinedStruct"));

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		// StructViewer Editor Settings
		SettingsModule->RegisterSettings("Project", "Editor", "StructViewer",
			LOCTEXT("StructViewerSettingsName", "Struct Viewer"),
			LOCTEXT("StructViewerSettingsDescription", "Configure options for the Struct Viewer."),
			GetMutableDefault<UStructViewerProjectSettings>()
			);
	}
}

void FStructViewerModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(StructViewerModule::StructViewerApp);
	}

	// Unregister the setting
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Editor", "StructViewer");
	}

	SStructViewer::DestroyStructHierarchy();
}

TSharedRef<SWidget> FStructViewerModule::CreateStructViewer(const FStructViewerInitializationOptions& InitOptions, const FOnStructPicked& OnStructPickedDelegate)
{
	return SNew(SStructViewer, InitOptions)
		.OnStructPickedDelegate(OnStructPickedDelegate);
}

#undef LOCTEXT_NAMESPACE
