// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructViewerNode.h"

#include "AssetRegistry/AssetData.h"
#include "StructUtils/UserDefinedStruct.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopedSlowTask.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#define LOCTEXT_NAMESPACE "StructViewer"

FStructViewerNodeData::FStructViewerNodeData()
	: StructName(TEXT("None"))
	, StructDisplayName(LOCTEXT("None", "None"))
{
}

FStructViewerNodeData::FStructViewerNodeData(const UScriptStruct* InStruct)
	: StructName(InStruct->GetName())
	, StructDisplayName(InStruct->GetDisplayNameText())
	, StructPath(*InStruct->GetPathName())
	, Struct(InStruct)
{
	if (const UScriptStruct* ParentStruct = Cast<UScriptStruct>(InStruct->GetSuperStruct()))
	{
		ParentStructPath = *ParentStruct->GetPathName();
	}
}

FStructViewerNodeData::FStructViewerNodeData(const FAssetData& InStructAsset)
	: StructName(InStructAsset.AssetName.ToString())
	, StructPath(InStructAsset.GetSoftObjectPath())
{
	// Attempt to find the struct asset in the case where it's already been loaded
	Struct = FindObject<UScriptStruct>(nullptr, *StructPath.ToString());

	// Cache the resolved display name if available, or synthesize one if the struct asset is unloaded
	StructDisplayName = Struct.IsValid()
		? Struct->GetDisplayNameText()
		: FText::AsCultureInvariant(FName::NameToDisplayString(StructName, false));
}

const UScriptStruct* FStructViewerNodeData::GetStruct() const
{
	return Struct.Get();
}

const UUserDefinedStruct* FStructViewerNodeData::GetStructAsset() const
{
	return Cast<UUserDefinedStruct>(Struct.Get());
}

bool FStructViewerNodeData::LoadStruct() const
{
	if (Struct.IsValid())
	{
		return true;
	}

	// Attempt to load the struct
	if (!StructPath.IsNull())
	{
		FScopedSlowTask SlowTask(0.0f, LOCTEXT("LoadingStruct", "Loading Struct..."));
		SlowTask.MakeDialogDelayed(1.0f);

		Struct = LoadObject<UScriptStruct>(nullptr, *StructPath.ToString());
	}

	// Re-cache the resolved display name as it may be different than the one we synthesized for an unloaded struct asset
	if (Struct.IsValid())
	{
		StructDisplayName = Struct->GetDisplayNameText();
		return true;
	}

	return false;
}

void FStructViewerNodeData::AddChild(const TSharedRef<FStructViewerNodeData>& InChild)
{
	InChild->ParentNode = AsShared();
	ChildNodes.Add(InChild);
}

void FStructViewerNodeData::AddUniqueChild(const TSharedRef<FStructViewerNodeData>& InChild)
{
	const bool bAlreadyAChild = ChildNodes.ContainsByPredicate([&InChild](const TSharedPtr<FStructViewerNodeData>& InExistingChild) -> bool
	{
		return InExistingChild->GetStructPath() == InChild->GetStructPath();
	});

	if (!bAlreadyAChild)
	{
		AddChild(InChild);
	}
}

bool FStructViewerNodeData::RemoveChild(const FSoftObjectPath& InStructPath)
{
	for (auto ChildIt = ChildNodes.CreateIterator(); ChildIt; ++ChildIt)
	{
		if ((*ChildIt)->GetStructPath() == InStructPath)
		{
			ChildIt.RemoveCurrent();
			return true;
		}
	}

	return false;
}


FStructViewerNode::FStructViewerNode()
	: NodeData(MakeShared<FStructViewerNodeData>())
	, bPassedFilter(true)
{
}

FStructViewerNode::FStructViewerNode(const TSharedRef<FStructViewerNodeData>& InData, const TSharedPtr<IPropertyHandle>& InPropertyHandle, const bool InPassedFilter)
	: NodeData(InData)
	, PropertyHandle(InPropertyHandle)
	, bPassedFilter(InPassedFilter)
{
}

FText FStructViewerNode::GetStructDisplayName(const EStructViewerNameTypeToDisplay InNameType) const
{
	switch (InNameType)
	{
	case EStructViewerNameTypeToDisplay::StructName:
		return FText::AsCultureInvariant(GetStructName());

	case EStructViewerNameTypeToDisplay::DisplayName:
		return GetStructDisplayName();

	case EStructViewerNameTypeToDisplay::Dynamic:
		{
			const FText BasicName = FText::AsCultureInvariant(GetStructName());
			const FText DisplayName = GetStructDisplayName();
			const FString SynthesizedDisplayName = FName::NameToDisplayString(BasicName.ToString(), false);

			// Only show both names if we have a display name set that is different from the basic name and not synthesized from it
			return (DisplayName.IsEmpty() || DisplayName.ToString().Equals(BasicName.ToString()) || DisplayName.ToString().Equals(SynthesizedDisplayName))
				? BasicName
				: FText::Format(LOCTEXT("StructDynamicDisplayNameFmt", "{0} ({1})"), BasicName, DisplayName);
		}

	default:
		break;
	}

	ensureMsgf(false, TEXT("FStructViewerNode::GetStructName called with invalid name type."));
	return GetStructDisplayName();
}

void FStructViewerNode::AddChild(const TSharedRef<FStructViewerNode>& InChild)
{
	InChild->ParentNode = AsShared();
	ChildNodes.Add(InChild);
}

void FStructViewerNode::AddUniqueChild(const TSharedRef<FStructViewerNode>& InChild)
{
	const bool bAlreadyAChild = ChildNodes.ContainsByPredicate([&InChild](const TSharedPtr<FStructViewerNode>& InExistingChild) -> bool
	{
		return InExistingChild->GetStructPath() == InChild->GetStructPath();
	});

	if (!bAlreadyAChild)
	{
		AddChild(InChild);
	}
}

void FStructViewerNode::SortChildrenRecursive()
{
	SortChildren();
	for (const TSharedPtr<FStructViewerNode>& ChildNode : ChildNodes)
	{
		ChildNode->SortChildrenRecursive();
	}
}

void FStructViewerNode::SortChildren()
{
	SortNodes(ChildNodes, PropertyHandle);
}

bool FStructViewerNode::SortPredicate(const TSharedPtr<FStructViewerNode>& InA, const TSharedPtr<FStructViewerNode>& InB)
{
	check(InA.IsValid());
	check(InB.IsValid());

	const FString& AString = InA->GetStructName();
	const FString& BString = InB->GetStructName();

	return AString < BString;
}

void FStructViewerNode::SortNodes(TArray<TSharedPtr<FStructViewerNode>>& Nodes, const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (PropertyHandle.IsValid())
	{
		static const FName SortFunctionMetaName = "SortFunction";

		const FString& FunctionName = PropertyHandle->GetMetaData(SortFunctionMetaName);

		if (!FunctionName.IsEmpty())
		{
			const UFunction* Function = nullptr;

			// Fully-qualified function name
			if (UE::String::FindFirstChar(FunctionName, TEXT('.')) != INDEX_NONE)
			{
				Function = FindObject<UFunction>(nullptr, *FunctionName, EFindObjectFlags::ExactClass);
			}
			else
			{
				TArray<UObject*> Containers;
				PropertyHandle->GetOuterObjects(Containers);

				for (const UObject* Container : Containers)
				{
					Function = Container ? Container->FindFunction(FName(FunctionName)) : nullptr;
					if (Function)
					{
						break;
					}
				}
			}

			if (ensureMsgf(Function && Function->HasAnyFunctionFlags(EFunctionFlags::FUNC_Static), TEXT("Invalid SortFunction: %s"), *FunctionName))
			{
				DECLARE_DELEGATE_RetVal_TwoParams(bool, FStructViewerUserCompare, const UScriptStruct*, const UScriptStruct*);

				UObject* Container = Function->GetOuterUClass()->GetDefaultObject();
				FStructViewerUserCompare CompareFunc = FStructViewerUserCompare::CreateUFunction(Container, Function->GetFName());

				Nodes.Sort([&CompareFunc](const TSharedPtr<FStructViewerNode>& InA, const TSharedPtr<FStructViewerNode>& InB)
				{
					check(InA.IsValid() && InB.IsValid());

					const UScriptStruct* AStruct = InA->GetStruct();
					const UScriptStruct* BStruct = InB->GetStruct();
					if (AStruct && BStruct)
					{
						return CompareFunc.Execute(InA->GetStruct(), InB->GetStruct());
					}

					return SortPredicate(InA, InB);
				});

				return;
			}
		}
	}

	Nodes.Sort(&SortPredicate);
}

bool FStructViewerNode::IsRestricted() const
{
	return PropertyHandle && PropertyHandle->IsRestricted(GetStructName());
}

#undef LOCTEXT_NAMESPACE
