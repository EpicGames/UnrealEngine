// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "StructViewerModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IPropertyHandle;
class UScriptStruct;
class UUserDefinedStruct;
struct FAssetData;

/** Common data representing an unfiltered hierarchy of nodes */
class FStructViewerNodeData : public TSharedFromThis<FStructViewerNodeData>
{
public:
	/** Create a dummy node */
	FStructViewerNodeData();

	/** Create a node representing the given struct */
	explicit FStructViewerNodeData(const UScriptStruct* InStruct);

	/** Create a node representing the given struct asset (may be unloaded) */
	explicit FStructViewerNodeData(const FAssetData& InStructAsset);

	/** Get the unlocalized name of the struct we represent */
	const FString& GetStructName() const
	{
		return StructName;
	}

	/** Get the localized name of the struct we represent */
	FText GetStructDisplayName() const
	{
		return StructDisplayName;
	}

	/** Get the full object path to the struct we represent */
	FSoftObjectPath GetStructPath() const
	{
		return StructPath;
	}

	/** Get the full object path to the parent of the struct we represent */
	FSoftObjectPath GetParentStructPath() const
	{
		return ParentStructPath;
	}

	/** Get the struct that we represent (for loaded struct assets, or native structs) */
	const UScriptStruct* GetStruct() const;

	/** Get the struct asset that we represent (for loaded struct assets) */
	const UUserDefinedStruct* GetStructAsset() const;

	/**
	 * Trigger a load of the struct we represent.
	 * @return True if the struct was loaded (or was already loaded), false otherwise.
	 */
	bool LoadStruct() const;

	/** Get the pointer to the parent data node (if any) */
	TSharedPtr<FStructViewerNodeData> GetParentNode() const
	{
		return ParentNode.Pin();
	}

	/** Get the list of child data nodes */
	const TArray<TSharedPtr<FStructViewerNodeData>>& GetChildNodes() const
	{
		return ChildNodes;
	}

	/** Adds the specified child to this node */
	void AddChild(const TSharedRef<FStructViewerNodeData>& InChild);

	/** Adds the specified child to this node, only if a node representing the struct doesn't already exist as a child */
	void AddUniqueChild(const TSharedRef<FStructViewerNodeData>& InChild);

	/** Remove the child representing the given struct path (if present) */
	bool RemoveChild(const FSoftObjectPath& InStructPath);

private:
	/** The unlocalized name of the struct we represent */
	FString StructName;

	/** The localized name of the struct we represent */
	mutable FText StructDisplayName;

	/** The full object path to the struct we represent */
	FSoftObjectPath StructPath;

	/** The full object path to the parent of the struct we represent */
	FSoftObjectPath ParentStructPath;

	/** The struct that we represent (for loaded struct assets, or native structs) */
	mutable TWeakObjectPtr<const UScriptStruct> Struct;

	/** Pointer to the parent data node (if any) */
	TWeakPtr<FStructViewerNodeData> ParentNode;

	/** List of child data nodes */
	TArray<TSharedPtr<FStructViewerNodeData>> ChildNodes;
};

/** Filtered data representing a filtered hierarchy of nodes */
class FStructViewerNode : public TSharedFromThis<FStructViewerNode>
{
public:
	/** Create a dummy node */
	FStructViewerNode();

	/** Create a node representing the given data */
	FStructViewerNode(const TSharedRef<FStructViewerNodeData>& InData, const TSharedPtr<IPropertyHandle>& InPropertyHandle, const bool InPassedFilter);

	/** Get the unlocalized name of the struct we represent */
	const FString& GetStructName() const
	{
		return NodeData->GetStructName();
	}

	/** Get the localized name of the struct we represent */
	FText GetStructDisplayName() const
	{
		return NodeData->GetStructDisplayName();
	}

	/** Get the display name of the struct we represent, built based on the given option */
	FText GetStructDisplayName(const EStructViewerNameTypeToDisplay InNameType) const;

	/** Get the full object path to the struct we represent */
	FSoftObjectPath GetStructPath() const
	{
		return NodeData->GetStructPath();
	}

	/** Get the full object path to the parent of the struct we represent */
	FSoftObjectPath GetParentStructPath() const
	{
		return NodeData->GetParentStructPath();
	}

	/** Get the struct that we represent (for loaded struct assets, or native structs) */
	const UScriptStruct* GetStruct() const
	{
		return NodeData->GetStruct();
	}

	/** Get the struct asset that we represent (for loaded struct assets) */
	const UUserDefinedStruct* GetStructAsset() const
	{
		return NodeData->GetStructAsset();
	}

	/**
	 * Trigger a load of the struct we represent.
	 * @return True if the struct was loaded (or was already loaded), false otherwise.
	 */
	bool LoadStruct() const
	{
		return NodeData->LoadStruct();
	}

	/** Get the pointer to the parent node (if any) */
	TSharedPtr<FStructViewerNode> GetParentNode() const
	{
		return ParentNode.Pin();
	}

	/** Get the list of child nodes */
	const TArray<TSharedPtr<FStructViewerNode>>& GetChildNodes() const
	{
		return ChildNodes;
	}

	/** Adds the specified child to this node */
	void AddChild(const TSharedRef<FStructViewerNode>& InChild);

	/** Adds the specified child to this node, only if a node representing the struct doesn't already exist as a child */
	void AddUniqueChild(const TSharedRef<FStructViewerNode>& InChild);

	/** Sort the child nodes based on FStructViewerNode::SortPredicate */
	void SortChildren();

	/** Sort the child nodes recursively based on FStructViewerNode::SortPredicate */
	void SortChildrenRecursive();

	/** Predicate function that can be used to sort instances by struct name */
	static bool SortPredicate(const TSharedPtr<FStructViewerNode>& InA, const TSharedPtr<FStructViewerNode>& InB);

	/** Sorts the array of nodes either by a custom sort function, or by using the default one (::SortPredicate) */
	static void SortNodes(TArray<TSharedPtr<FStructViewerNode>>& Nodes, const TSharedPtr<IPropertyHandle>& PropertyHandle = { });

	/** Check whether this struct is restricted for the specific context */
	bool IsRestricted() const;

	/** Get the property this filtered node will be working on */
	const TSharedPtr<IPropertyHandle>& GetPropertyHandle() const
	{
		return PropertyHandle;
	}

	/** True if the struct passed the filter */
	bool PassedFilter() const
	{
		return bPassedFilter;
	}

	/** True if the struct passed the filter */
	void PassedFilter(const bool InPassedFilter)
	{
		bPassedFilter = InPassedFilter;
	}

private:
	/** Reference to the common data this filtered node represents */
	TSharedRef<const FStructViewerNodeData> NodeData;

	/** The property this filtered node will be working on */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** True if the struct passed the filter */
	bool bPassedFilter;

	/** Pointer to the parent node (if any) */
	TWeakPtr<FStructViewerNode> ParentNode;

	/** List of child nodes */
	TArray<TSharedPtr<FStructViewerNode>> ChildNodes;
};
