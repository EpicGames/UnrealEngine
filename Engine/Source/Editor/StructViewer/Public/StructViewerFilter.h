// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#define UE_API STRUCTVIEWER_API

class FStructViewerInitializationOptions;

/** Interface class for creating filters for the Struct Viewer. */
class IStructViewerFilter
{
public:
	virtual ~IStructViewerFilter() = default;

	/**
	 * Checks if a struct is allowed by this filter.
	 *
	 * @param InInitOptions				The Struct Viewer/Picker options.
	 * @param InStruct					The struct to be tested.
	 * @param InFilterFuncs				Useful functions for filtering.
	 */
	virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs) = 0;

	/**
	 * Checks if a struct is allowed by this filter.
	 *
	 * @param InInitOptions				The Struct Viewer/Picker options.
	 * @param InStructPath				The path of the unloaded struct to be tested.
	 * @param InFilterFuncs				Useful functions for filtering.
	 */
	virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs) = 0;
};

enum class EStructFilterReturn : uint8
{
	Failed = 0, 
	Passed, 
	NoItems,
};
class FStructViewerFilterFuncs
{
public:
	virtual ~FStructViewerFilterFuncs() = default;

	/** 
	 * Checks if the given struct is a child-of any of the structs in a set.
	 *
	 * @param InSet				The set to test against.
	 * @param InStruct			The struct to test against.
	 *
	 * @return					EFilterReturn::Passed if it is a child-of a struct in the set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EStructFilterReturn IfInChildOfStructsSet(TSet<const UScriptStruct*>& InSet, const UScriptStruct* InStruct);

	/** 
	 * Checks if the given struct is a child-of ALL of the classes in a set.
	 *
	 * @param InSet				The set to test against.
	 * @param InStruct			The struct to test against.
	 *
	 * @return					EFilterReturn::Passed if it is a child-of ALL the structs in the set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EStructFilterReturn IfMatchesAllInChildOfStructsSet(TSet<const UScriptStruct*>& InSet, const UScriptStruct* InStruct);

	/** 
	 * Checks if the struct is in the structs set.
	 *
	 * @param InSet				The set to test against.
	 * @param InStruct			The struct to test against.
	 *
	 * @return					EFilterReturn::Passed if the struct is in the structs set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EStructFilterReturn IfInStructsSet(TSet<const UScriptStruct*>& InSet, const UScriptStruct* InStruct);
};

#undef UE_API
