// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class IStructViewerFilter;
class UScriptStruct;

/** Delegate used with the Struct Viewer in 'struct picking' mode.  You'll bind a delegate when the struct viewer widget is created, which will be fired off when a struct is selected in the list */
DECLARE_DELEGATE_OneParam(FOnStructPicked, const UScriptStruct*);
/** Delegate used with the Struct Viewer in 'struct picking' mode.  You'll bind a delegate when the struct viewer widget is created, which will be fired off when a list of structs is opened for selection */
DECLARE_DELEGATE_OneParam(FOnStructPickerOpened, const UScriptStruct*);

enum class EStructViewerMode : uint8
{
	/** Allows all structs to be browsed and selected; syncs selection with the editor; drag and drop attachment, etc. */
	StructBrowsing,

	/** Sets the struct viewer to operate as a struct 'picker'. */
	StructPicker,
};

enum class EStructViewerDisplayMode : uint8
{
	/** Default will choose what view mode based on if in Viewer or Picker mode. */
	DefaultView,

	/** Displays all classes as a tree. */
	TreeView,

	/** Displays all classes as a list. */
	ListView,
};

enum class EStructViewerNameTypeToDisplay : uint8
{
	/** Display both the display name and struct name if they're available and different. */
	Dynamic,

	/** Always use the display name */
	DisplayName,

	/** Always use the struct name */
	StructName,
};

/**
 * Settings for the Struct Viewer set by the programmer before spawning an instance of the widget.  This
 * is used to modify the struct viewer's behavior in various ways, such as filtering in or out specific structs.
 */
class FStructViewerInitializationOptions
{
public:
	/** The filter to use on structs in this instance. */
	TSharedPtr<class IStructViewerFilter> StructFilter;

	/** Mode to operate in */
	EStructViewerMode Mode;

	/** Mode to display the structs using. */
	EStructViewerDisplayMode DisplayMode;

	/** Shows unloaded structs. Will not be filtered out based on non-bool filter options. */
	bool bShowUnloadedStructs;

	/** Shows a "None" option, only available in Picker mode. */
	bool bShowNoneOption;

	/** If true, root nodes will be expanded by default. */
	bool bExpandRootNodes;

	/** true allows struct dynamic loading on selection */
	bool bEnableStructDynamicLoading;

	/** Controls what name is shown for structs */
	EStructViewerNameTypeToDisplay NameTypeToDisplay;

	/** the title string of the struct viewer if required. */
	FText ViewerTitleString;

	/** The property this struct viewer be working on. */
	TSharedPtr<class IPropertyHandle> PropertyHandle;

	/** true (the default) shows the view options at the bottom of the struct picker. */
	bool bAllowViewOptions;

	/** true (the default) shows a background border behind the struct viewer widget. */
	bool bShowBackgroundBorder;

	/** Defined currently selected struct to scroll to to when struct picker is opened. */
	const UScriptStruct* SelectedStruct;

	/** Defines additional structs you want listed in the "Common Structs" section for the picker. */
	TArray<const UScriptStruct*> ExtraPickerCommonStructs;

public:
	/** Constructor */
	FStructViewerInitializationOptions()
		: Mode(EStructViewerMode::StructPicker)
		, DisplayMode(EStructViewerDisplayMode::DefaultView)
		, bShowUnloadedStructs(true)
		, bShowNoneOption(false)
		, bExpandRootNodes(true)
		, bEnableStructDynamicLoading(true)
		, NameTypeToDisplay(EStructViewerNameTypeToDisplay::StructName)
		, bAllowViewOptions(true)
		, bShowBackgroundBorder(true)
	    , SelectedStruct(nullptr)
	{
	}
};

/**
 * Struct Viewer module
 */
class FStructViewerModule : public IModuleInterface
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Creates a struct viewer widget
	 *
	 * @param	InitOptions						Programmer-driven configuration for this widget instance
	 * @param	OnStructPickedDelegate			Optional callback when a struct is selected in 'struct picking' mode
	 *
	 * @return	New struct viewer widget
	 */
	virtual TSharedRef<class SWidget> CreateStructViewer(const FStructViewerInitializationOptions& InitOptions, const FOnStructPicked& OnStructPickedDelegate);
};
