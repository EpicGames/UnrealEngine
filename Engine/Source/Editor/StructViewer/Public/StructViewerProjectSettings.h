// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Engine/EngineTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "StructViewerProjectSettings.generated.h"

class UScriptStruct;

/**
 * Implements the settings for the Struct Viewer Project Settings
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UStructViewerProjectSettings : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** The base directories to be considered Internal Only for the struct picker.*/
	UPROPERTY(EditAnywhere, config, Category = StructVisibilityManagement, meta = (DisplayName = "List of directories to consider Internal Only.", ContentDir, LongPackageName))
	TArray<FDirectoryPath> InternalOnlyPaths;

	/** The base classes to be considered Internal Only for the struct picker.*/
	UPROPERTY(EditAnywhere, config, Category = StructVisibilityManagement, meta = (DisplayName = "List of base structs to consider Internal Only.", ShowTreeView, HideViewOptions))
	TArray<TSoftObjectPtr<const UScriptStruct>> InternalOnlyStructs;
#endif
};
