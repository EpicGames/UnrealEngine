// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class StructViewer : ModuleRules
{
	public StructViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"EditorWidgets",
				"WorkspaceMenuStructure",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"ContentBrowserData",
				"Settings",
				"ApplicationCore",
            }
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
                "AssetTools",
				"EditorWidgets",
			}
		);
	}
}
